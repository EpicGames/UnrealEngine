[CoreRedirects]
+ClassRedirects=(OldName="/Script/CustomPropertyControl.CustomPropertyControlResolver",NewName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreResolver")
+ClassRedirects=(OldName="/Script/CustomPropertyControl.CustomPropertyControlHandlerBase",NewName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreHandlerBase")
+ClassRedirects=(OldName="/Script/CustomPropertyControl.CustomPropertyControlFloatHandler",NewName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreFloatHandler")
+ClassRedirects=(OldName="/Script/CustomPropertyControl.CustomPropertyControlDoubleHandler",NewName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreDoubleHandler")
+ClassRedirects=(OldName="/Script/CustomPropertyControl.CustomPropertyControlControllerBase",NewName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreBase")
+ClassRedirects=(OldName="/Script/CustomPropertyControl.CustomPropertyControlComponent",NewName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreComponent")
+ClassRedirects=(OldName="/Script/PropertyController.PropertyControllerCustomTimeSource",NewName="/Script/PropertyAnimator.PropertyAnimatorCustomTimeSource")
+ClassRedirects=(OldName="/Script/PropertyController.PropertyControllerTimeSourceBase",NewName="/Script/PropertyAnimator.PropertyAnimatorTimeSourceBase")
+ClassRedirects=(OldName="/Script/PropertyController.PropertyControllerWorldTimeSource",NewName="/Script/PropertyAnimator.PropertyAnimatorWorldTimeSource")
+ClassRedirects=(OldName="/Script/PropertyAnimator.PropertyAnimatorCustomTimeSource",NewName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreManualTimeSource")
+ClassRedirects=(OldName="/Script/PropertyAnimator.PropertyAnimatorTimeSourceBase",NewName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreTimeSourceBase")
+ClassRedirects=(OldName="/Script/PropertyAnimator.PropertyAnimatorWorldTimeSource",NewName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreWorldTimeSource")
+ClassRedirects=(OldName="/Script/AvalanchePropertyAnimator.AvaPropertyAnimatorSequenceTimeSource",NewName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreSequencerTimeSource")
+ClassRedirects=(OldName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreMachineClockTimeSource",NewName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreSystemTimeSource")

+StructRedirects=(OldName="/Script/CustomPropertyControlEditor.CustomPropertyControlData",NewName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreData")

+PropertyRedirects=(OldName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreBase.ControllerDisplayName",NewName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreBase.AnimatorDisplayName")
+PropertyRedirects=(OldName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreComponent.Controllers",NewName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreComponent.Animators")
+PropertyRedirects=(OldName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreBase.bEnabled",NewName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreBase.bAnimatorEnabled")
+PropertyRedirects=(OldName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreComponent.bEnabled",NewName="/Script/PropertyAnimatorCore.PropertyAnimatorCoreComponent.bAnimatorsEnabled")

+EnumRedirects=(OldName="/Script/PropertyAnimatorCore.EPropertyAnimatorCoreMachineClockMode",NewName="/Script/PropertyAnimatorCore.EPropertyAnimatorCoreSystemMode")