{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Property Animator Core",
	"Description": "Re-usable behaviors to control properties at runtime and in editor",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "PropertyAnimatorCore",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "PropertyAnimatorCoreEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "OperatorStack",
			"Enabled": true
		}
	]
}