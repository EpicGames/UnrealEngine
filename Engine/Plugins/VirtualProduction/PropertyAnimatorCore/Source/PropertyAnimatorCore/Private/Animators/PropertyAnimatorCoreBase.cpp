// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animators/PropertyAnimatorCoreBase.h"

#include "Components/PropertyAnimatorCoreComponent.h"
#include "GameFramework/Actor.h"
#include "Logs/PropertyAnimatorCoreLog.h"
#include "Properties/PropertyAnimatorCoreResolver.h"
#include "Properties/Handlers/PropertyAnimatorCoreHandlerBase.h"
#include "Settings/PropertyAnimatorCoreSettings.h"
#include "Subsystems/PropertyAnimatorCoreSubsystem.h"
#include "TimeSources/PropertyAnimatorCoreTimeSourceBase.h"
#include "UObject/ObjectSaveContext.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

UPropertyAnimatorCoreBase::FOnAnimatorUpdated UPropertyAnimatorCoreBase::OnAnimatorAddedDelegate;
UPropertyAnimatorCoreBase::FOnAnimatorUpdated UPropertyAnimatorCoreBase::OnAnimatorRemovedDelegate;
UPropertyAnimatorCoreBase::FOnAnimatorUpdated UPropertyAnimatorCoreBase::OnAnimatorRenamedDelegate;
UPropertyAnimatorCoreBase::FOnAnimatorPropertyUpdated UPropertyAnimatorCoreBase::OnAnimatorPropertyLinkedDelegate;
UPropertyAnimatorCoreBase::FOnAnimatorPropertyUpdated UPropertyAnimatorCoreBase::OnAnimatorPropertyUnlinkedDelegate;

#if WITH_EDITOR
FName UPropertyAnimatorCoreBase::GetAnimatorEnabledPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreBase, bAnimatorEnabled);
}

FName UPropertyAnimatorCoreBase::GetLinkedPropertiesPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreBase, LinkedProperties);
}
#endif

UPropertyAnimatorCoreBase::UPropertyAnimatorCoreBase()
{
	if (!IsTemplate())
	{
		Metadata = GetClass()->GetDefaultObject<UPropertyAnimatorCoreBase>()->Metadata;

#if WITH_EDITOR
		FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UPropertyAnimatorCoreBase::OnObjectReplaced);
		FEditorDelegates::PreSaveWorldWithContext.AddUObject(this, &UPropertyAnimatorCoreBase::OnPreSaveWorld);
		FEditorDelegates::PreBeginPIE.AddUObject(this, &UPropertyAnimatorCoreBase::OnPreBeginPIE);
#endif
	}
}

UPropertyAnimatorCoreComponent* UPropertyAnimatorCoreBase::GetAnimatorComponent() const
{
	return GetTypedOuter<UPropertyAnimatorCoreComponent>();
}

void UPropertyAnimatorCoreBase::UpdateAnimatorDisplayName()
{
	TArray<FString> PropertiesNames;
	for (const FPropertyAnimatorCoreData& LinkedProperty : GetLinkedProperties())
	{
		PropertiesNames.Add(LinkedProperty.GetPropertyDisplayName());
	}

	auto FindCommonPrefix = [](const TConstArrayView<FString>& InNames)->FString
	{
		if (InNames.IsEmpty())
		{
			return FString();
		}

		FString CommonPrefix = InNames[0];

		for (int32 Index = 1; Index < InNames.Num(); ++Index)
		{
			const FString& CurrentString = InNames[Index];

			int32 CommonChars = 0;
			while (CommonChars < CommonPrefix.Len()
				&& CommonChars < CurrentString.Len()
				&& CommonPrefix[CommonChars] == CurrentString[CommonChars])
			{
				++CommonChars;
			}

			CommonPrefix = CommonPrefix.Left(CommonChars);
		}

		return CommonPrefix;
	};

	FString CommonPrefix = FindCommonPrefix(PropertiesNames);
	CommonPrefix = CommonPrefix.TrimChar(*TEXT("."));

	if (CommonPrefix.IsEmpty())
	{
		SetAnimatorDisplayName(GetFName());
	}
	else
	{
		SetAnimatorDisplayName(FName(GetAnimatorMetadata()->Name.ToString() + TEXT("_") + CommonPrefix));
	}
}

UPropertyAnimatorCoreContext* UPropertyAnimatorCoreBase::GetLinkedPropertyContext(const FPropertyAnimatorCoreData& InProperty) const
{
	const TObjectPtr<UPropertyAnimatorCoreContext>* PropertyOptions = LinkedProperties.FindByPredicate([&InProperty](const UPropertyAnimatorCoreContext* InOptions)
	{
		return InOptions && InOptions->GetAnimatedProperty() == InProperty;
	});

	return PropertyOptions ? *PropertyOptions : nullptr;
}

void UPropertyAnimatorCoreBase::PostCDOContruct()
{
	Super::PostCDOContruct();

	if (IsTemplate() && !Metadata.IsValid())
	{
		Metadata = MakeShared<FPropertyAnimatorCoreMetadata>();

#if WITH_EDITOR
		const UClass* AnimatorClass = GetClass();
		Metadata->DisplayName = AnimatorClass->GetDisplayNameText();
		Metadata->Description = AnimatorClass->GetToolTipText();
#endif

		OnAnimatorRegistered(*Metadata);

		if (Metadata->DisplayName.IsEmpty())
		{
			Metadata->DisplayName = FText::FromName(Metadata->Name);
		}

		SetAnimatorDisplayName(Metadata->Name);
	}
}

void UPropertyAnimatorCoreBase::BeginDestroy()
{
	Super::BeginDestroy();

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	FEditorDelegates::PreSaveWorldWithContext.RemoveAll(this);
	FEditorDelegates::PreBeginPIE.RemoveAll(this);
#endif
}

void UPropertyAnimatorCoreBase::PostLoad()
{
	Super::PostLoad();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	// Migrate deprecated property
	if (TimeSources.IsEmpty())
	{
		TimeSourcesInstances.GenerateValueArray(TimeSources);
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	OnTimeSourceNameChanged();

	CleanLinkedProperties();

	OnAnimatorEnabledChanged(EPropertyAnimatorCoreUpdateEvent::Load);
}

void UPropertyAnimatorCoreBase::PostEditImport()
{
	Super::PostEditImport();

	OnTimeSourceNameChanged();
	ResolvePropertiesOwner();
}

void UPropertyAnimatorCoreBase::PreDuplicate(FObjectDuplicationParameters& InParams)
{
	Super::PreDuplicate(InParams);

	RestoreProperties();
}

void UPropertyAnimatorCoreBase::PostDuplicate(EDuplicateMode::Type InMode)
{
	Super::PostDuplicate(InMode);

	OnTimeSourceNameChanged();
	ResolvePropertiesOwner();
}

#if WITH_EDITOR
void UPropertyAnimatorCoreBase::PreEditUndo()
{
	Super::PreEditUndo();

	RestoreProperties();
}

void UPropertyAnimatorCoreBase::PostEditUndo()
{
	Super::PostEditUndo();

	RestoreProperties();
}

void UPropertyAnimatorCoreBase::PreEditChange(FProperty* InPropertyAboutToChange)
{
	Super::PreEditChange(InPropertyAboutToChange);

	if (InPropertyAboutToChange
		&& InPropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreBase, bAnimatorEnabled))
	{
		RestoreProperties();
	}
}

void UPropertyAnimatorCoreBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreBase, bAnimatorEnabled))
	{
		OnAnimatorEnabledChanged(EPropertyAnimatorCoreUpdateEvent::User);
	}
	else if (MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreBase, TimeSourceName)
		|| MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreBase, bOverrideTimeSource))
	{
		OnTimeSourceNameChanged();
	}
}
#endif

bool UPropertyAnimatorCoreBase::ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue)
{
	if (!InValue->IsObject())
	{
		return false;
	}

	const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> AnimatorArchive = InValue->AsMutableObject();

	bool bEnabledValue = bAnimatorEnabled;
	AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreBase, bAnimatorEnabled), bEnabledValue);
	SetAnimatorEnabled(bEnabledValue);

	FString DisplayNameValue = AnimatorDisplayName.ToString();
	AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreBase, AnimatorDisplayName), DisplayNameValue);
	SetAnimatorDisplayName(FName(DisplayNameValue));

	TSharedPtr<FPropertyAnimatorCorePresetArchive> LinkedPropertiesArchive;
	AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreBase, LinkedProperties), LinkedPropertiesArchive);
	if (const TSharedPtr<FPropertyAnimatorCorePresetArrayArchive> LinkedPropertiesArray = LinkedPropertiesArchive->AsMutableArray())
	{
		for (int32 Index = 0; Index < LinkedPropertiesArray->Num(); Index++)
		{
			TSharedPtr<FPropertyAnimatorCorePresetArchive> LinkedPropertyArchive;
			if (!LinkedPropertiesArray->Get(Index, LinkedPropertyArchive) || !LinkedPropertyArchive->IsObject())
			{
				continue;
			}

			const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> LinkedPropertyObject = LinkedPropertyArchive->AsMutableObject();

			FString AnimatedPropertyLocatorPath;
			if (!LinkedPropertyObject->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreContext, AnimatedProperty), AnimatedPropertyLocatorPath))
			{
				continue;
			}

			FPropertyAnimatorCoreData PropertyData(GetAnimatorActor(), AnimatedPropertyLocatorPath);

			if (!PropertyData.IsResolved())
			{
				continue;
			}

			if (IPropertyAnimatorCorePresetable* PropertyContext = Cast<IPropertyAnimatorCorePresetable>(LinkProperty(PropertyData)))
			{
				PropertyContext->ImportPreset(InPreset, LinkedPropertyArchive.ToSharedRef());
			}
		}
	}

	bool bOverrideTimeSourceValue = bOverrideTimeSource;
	AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreBase, bOverrideTimeSource), bOverrideTimeSourceValue);
	SetOverrideTimeSource(bOverrideTimeSourceValue);

	FString TimeSourceNameValue = TimeSourceName.ToString();
	AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreBase, TimeSourceName), TimeSourceNameValue);
	SetTimeSourceName(FName(TimeSourceNameValue));

	if (UPropertyAnimatorCoreTimeSourceBase* TimeSource = FindOrAddTimeSource(GetTimeSourceName()))
	{
		if (AnimatorArchive->Has(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreBase, ActiveTimeSource), EPropertyAnimatorCorePresetArchiveType::Object))
		{
			TSharedPtr<FPropertyAnimatorCorePresetArchive> TimeSourceArchive;
			AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreBase, ActiveTimeSource), TimeSourceArchive);
			TimeSource->ImportPreset(InPreset, TimeSourceArchive.ToSharedRef());
		}
	}

	return true;
}

bool UPropertyAnimatorCoreBase::ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const
{
	TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> AnimatorArchive = InPreset->GetArchiveImplementation()->CreateObject();
	OutValue = AnimatorArchive;

	AnimatorArchive->Set(TEXT("AnimatorClass"), GetClass()->GetClassPathName().ToString());
	AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreBase, bAnimatorEnabled), bAnimatorEnabled);
	AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreBase, AnimatorDisplayName), AnimatorDisplayName.ToString());

	TSharedPtr<FPropertyAnimatorCorePresetArrayArchive> LinkedPropertiesArchive = InPreset->GetArchiveImplementation()->CreateArray();
	for (const TObjectPtr<UPropertyAnimatorCoreContext>& LinkedProperty : LinkedProperties)
	{
		if (IPropertyAnimatorCorePresetable* PropertyContext = Cast<IPropertyAnimatorCorePresetable>(LinkedProperty))
		{
			TSharedPtr<FPropertyAnimatorCorePresetArchive> LinkedPropertyArchive;
			if (PropertyContext->ExportPreset(InPreset, LinkedPropertyArchive) && LinkedPropertyArchive.IsValid())
			{
				LinkedPropertiesArchive->Add(LinkedPropertyArchive.ToSharedRef());
			}
		}
	}
	AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreBase, LinkedProperties), LinkedPropertiesArchive.ToSharedRef());

	AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreBase, bOverrideTimeSource), bOverrideTimeSource);
	AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreBase, TimeSourceName), TimeSourceName.ToString());

	if (UPropertyAnimatorCoreTimeSourceBase* TimeSource = GetActiveTimeSource())
	{
		TSharedPtr<FPropertyAnimatorCorePresetArchive> TimeSourceArchive;
		if (TimeSource->ExportPreset(InPreset, TimeSourceArchive) && TimeSourceArchive.IsValid())
		{
			AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreBase, ActiveTimeSource), TimeSourceArchive.ToSharedRef());
		}
	}

	return true;
}

AActor* UPropertyAnimatorCoreBase::GetAnimatorActor() const
{
	return GetTypedOuter<AActor>();
}

void UPropertyAnimatorCoreBase::SetAnimatorEnabled(bool bInIsEnabled)
{
	if (bAnimatorEnabled == bInIsEnabled)
	{
		return;
	}

	bAnimatorEnabled = bInIsEnabled;
	OnAnimatorEnabledChanged(EPropertyAnimatorCoreUpdateEvent::User);
}

void UPropertyAnimatorCoreBase::SetOverrideTimeSource(bool bInOverride)
{
	if (bOverrideTimeSource == bInOverride)
	{
		return;
	}

	bOverrideTimeSource = bInOverride;
	OnTimeSourceNameChanged();
}

void UPropertyAnimatorCoreBase::SetTimeSourceName(FName InTimeSourceName)
{
	if (TimeSourceName == InTimeSourceName)
	{
		return;
	}

	const TArray<FName> TimeSourceNames = GetTimeSourceNames();
	if (!TimeSourceNames.Contains(InTimeSourceName))
	{
		return;
	}

	TimeSourceName = InTimeSourceName;
	OnTimeSourceNameChanged();
}

TSharedRef<const FPropertyAnimatorCoreMetadata> UPropertyAnimatorCoreBase::GetAnimatorMetadata() const
{
	checkf(Metadata.IsValid(), TEXT("Animator metadata is invalid"))
	return Metadata.ToSharedRef();
}

bool UPropertyAnimatorCoreBase::GetPropertiesSupported(const FPropertyAnimatorCoreData& InPropertyData, TSet<FPropertyAnimatorCoreData>& OutProperties, uint8 InSearchDepth, EPropertyAnimatorPropertySupport InSupportExpected) const
{
	const FProperty* LeafProperty = InPropertyData.GetLeafProperty();
	UObject* Owner = InPropertyData.GetOwner();

	// Is property editable
	if (!LeafProperty->HasAnyPropertyFlags(EPropertyFlags::CPF_Edit))
	{
		return false;
	}

	// We can directly control the member property
	if (HasPropertySupport(InPropertyData, InSupportExpected))
	{
		OutProperties.Add(InPropertyData);
	}

	if (--InSearchDepth == 0)
	{
		return !OutProperties.IsEmpty();
	}

	// Look for inner properties that can be controlled too
	TFunction<bool(TArray<FProperty*>&, UObject*, TSet<FPropertyAnimatorCoreData>&)> FindSupportedPropertiesRecursively = [this, &FindSupportedPropertiesRecursively, &InSearchDepth, &InPropertyData, InSupportExpected](TArray<FProperty*>& InChainProperties, UObject* InOwner, TSet<FPropertyAnimatorCoreData>& OutSupportedProperties)
	{
		if (InSearchDepth-- > 0)
		{
			FProperty* InLeafProperty = InChainProperties.Last();

			if (const FStructProperty* StructProp = CastField<FStructProperty>(InLeafProperty))
			{
				for (FProperty* Property : TFieldRange<FProperty>(StructProp->Struct))
				{
					if (!Property->HasAnyPropertyFlags(EPropertyFlags::CPF_Edit))
					{
						continue;
					}

					// Copy over resolver if any on that property
					FPropertyAnimatorCoreData PropertyControlData(InOwner, InChainProperties, Property, InPropertyData.GetPropertyResolverClass());

					// We can directly control this property
					if (HasPropertySupport(PropertyControlData, InSupportExpected))
					{
						OutSupportedProperties.Add(PropertyControlData);
					}

					// Check nested properties inside this property
					TArray<FProperty*> NestedChainProperties(InChainProperties);
					NestedChainProperties.Add(Property);
					FindSupportedPropertiesRecursively(NestedChainProperties, InOwner, OutSupportedProperties);
				}
			}
		}

		return !OutSupportedProperties.IsEmpty();
	};

	TArray<FProperty*> ChainProperties = InPropertyData.GetChainProperties();
	return FindSupportedPropertiesRecursively(ChainProperties, Owner, OutProperties);
}

EPropertyAnimatorPropertySupport UPropertyAnimatorCoreBase::GetPropertySupport(const FPropertyAnimatorCoreData& InPropertyData) const
{
	// Without any handler we can't control the property type
	if (!InPropertyData.GetPropertyHandler())
	{
		return EPropertyAnimatorPropertySupport::None;
	}

	return IsPropertySupported(InPropertyData);
}

bool UPropertyAnimatorCoreBase::HasPropertySupport(const FPropertyAnimatorCoreData& InPropertyData, EPropertyAnimatorPropertySupport InSupportExpected) const
{
	return EnumHasAnyFlags(InSupportExpected, GetPropertySupport(InPropertyData));
}

void UPropertyAnimatorCoreBase::OnAnimatorAdded(EPropertyAnimatorCoreUpdateEvent InType)
{
	if (InType == EPropertyAnimatorCoreUpdateEvent::User)
	{
		bOverrideTimeSource = false;

		if (const UPropertyAnimatorCoreSettings* AnimatorSettings = UPropertyAnimatorCoreSettings::Get())
		{
			SetTimeSourceName(AnimatorSettings->GetDefaultTimeSourceName());
		}
	}

	UPropertyAnimatorCoreBase::OnAnimatorAddedDelegate.Broadcast(GetAnimatorComponent(), this, InType);
}

void UPropertyAnimatorCoreBase::OnAnimatorRemoved(EPropertyAnimatorCoreUpdateEvent InType)
{
	UPropertyAnimatorCoreBase::OnAnimatorRemovedDelegate.Broadcast(GetAnimatorComponent(), this, InType);
}

void UPropertyAnimatorCoreBase::OnAnimatorEnabled(EPropertyAnimatorCoreUpdateEvent InType)
{
	UE_LOGF(LogPropertyAnimatorCore
		, Verbose
		, "%ls : PropertyAnimator %ls (%ls) enabled %i"
		, GetAnimatorActor() ? *GetAnimatorActor()->GetActorNameOrLabel() : TEXT("Invalid Actor")
		, *GetAnimatorDisplayName().ToString()
		, *GetAnimatorMetadata()->Name.ToString()
		, EnumToUnderlyingType(InType));
}

void UPropertyAnimatorCoreBase::OnAnimatorDisabled(EPropertyAnimatorCoreUpdateEvent InType)
{
	UE_LOGF(LogPropertyAnimatorCore
		, Verbose
		, "%ls : PropertyAnimator %ls (%ls) disabled %i"
		, GetAnimatorActor() ? *GetAnimatorActor()->GetActorNameOrLabel() : TEXT("Invalid Actor")
		, *GetAnimatorDisplayName().ToString()
		, *GetAnimatorMetadata()->Name.ToString()
		, EnumToUnderlyingType(InType));

	RestoreProperties();
}

TSubclassOf<UPropertyAnimatorCoreContext> UPropertyAnimatorCoreBase::GetPropertyContextClass(const FPropertyAnimatorCoreData& InProperty)
{
	return UPropertyAnimatorCoreContext::StaticClass();
}

void UPropertyAnimatorCoreBase::OnAnimatorEnabledChanged(EPropertyAnimatorCoreUpdateEvent InType)
{
	const UPropertyAnimatorCoreComponent* AnimatorComponent = GetAnimatorComponent();

	if (bAnimatorEnabled && AnimatorComponent->ShouldAnimate())
	{
		OnAnimatorEnabled(InType);
	}
	else
	{
		OnAnimatorDisabled(InType);
	}
}

void UPropertyAnimatorCoreBase::CleanLinkedProperties()
{
	for (TArray<TObjectPtr<UPropertyAnimatorCoreContext>>::TIterator It(LinkedProperties); It; ++It)
	{
		const UPropertyAnimatorCoreContext* PropertyContext = It->Get();
		if (!PropertyContext || !PropertyContext->GetAnimatedProperty().IsResolved())
		{
			It.RemoveCurrent();
		}
	}
}

void UPropertyAnimatorCoreBase::OnTimeSourceNameChanged()
{
	if (ActiveTimeSource)
	{
		ActiveTimeSource->DeactivateTimeSource();
	}

	ActiveTimeSource = bOverrideTimeSource ? FindOrAddTimeSource(TimeSourceName) : nullptr;

	if (ActiveTimeSource)
	{
		ActiveTimeSource->ActivateTimeSource();
	}

	OnTimeSourceChanged();
}

void UPropertyAnimatorCoreBase::OnTimeSourceEnterIdleState()
{
	RestoreProperties();
}

void UPropertyAnimatorCoreBase::ResolvePropertiesOwner(AActor* InNewOwner)
{
	// Resolve linked properties against current actor
	TSet<FPropertyAnimatorCoreData> UnresolvedProperties;

	ForEachLinkedProperty<UPropertyAnimatorCoreContext>(
		[this, &UnresolvedProperties, &InNewOwner](UPropertyAnimatorCoreContext* InContext, const FPropertyAnimatorCoreData& InProperty)->bool
		{
			if (!InContext->ResolvePropertyOwner(InNewOwner))
			{
				UnresolvedProperties.Add(InProperty);
			}

			return true;
		}, false);

	// Remove unresolved properties
	for (const FPropertyAnimatorCoreData& UnresolvedProperty : UnresolvedProperties)
	{
		UnlinkProperty(UnresolvedProperty);
	}
}

void UPropertyAnimatorCoreBase::EvaluateAnimator(FInstancedPropertyBag& InParameters)
{
	SaveProperties();

	EvaluatedPropertyValues.Reset();

	bEvaluatingProperties = true;
	EvaluateProperties(InParameters);
	bEvaluatingProperties = false;
}

void UPropertyAnimatorCoreBase::OnObjectReplaced(const TMap<UObject*, UObject*>& InReplacementMap)
{
	constexpr bool bResolve = false;
	ForEachLinkedProperty<UPropertyAnimatorCoreContext>([&InReplacementMap](UPropertyAnimatorCoreContext* InContext, const FPropertyAnimatorCoreData& InProperty)->bool
	{
		const TWeakObjectPtr<UObject> OwnerWeak = InProperty.GetOwnerWeak();

		constexpr bool bEvenIfPendingKill = true;
		const UObject* Owner = OwnerWeak.Get(bEvenIfPendingKill);

		if (UObject* const* NewOwner = InReplacementMap.Find(Owner))
		{
			InContext->SetAnimatedPropertyOwner(*NewOwner);
		}

		return true;
	}, bResolve);
}

#if WITH_EDITOR
void UPropertyAnimatorCoreBase::OnPreSaveWorld(UWorld* InWorld, FObjectPreSaveContext InContext)
{
	if (InWorld != GetWorld())
	{
		return;
	}

	RestoreProperties();
}

void UPropertyAnimatorCoreBase::OnPreBeginPIE(bool bInSimulating)
{
	RestoreProperties();
}
#endif

void UPropertyAnimatorCoreBase::RestoreProperties()
{
	constexpr bool bResolve = false;
	bool bResult = ForEachLinkedProperty<UPropertyAnimatorCoreContext>([](UPropertyAnimatorCoreContext* InOptions, const FPropertyAnimatorCoreData& InPropertyData)
	{
		InOptions->Restore();
		return true;
	}, bResolve);
}

void UPropertyAnimatorCoreBase::SaveProperties()
{
	constexpr bool bResolve = false;
	bool bResult = ForEachLinkedProperty<UPropertyAnimatorCoreContext>([](UPropertyAnimatorCoreContext* InOptions, const FPropertyAnimatorCoreData& InPropertyData)
	{
		InOptions->Save();
		return true;
	}, bResolve);
}

TArray<FName> UPropertyAnimatorCoreBase::GetTimeSourceNames() const
{
	TArray<FName> TimeSourceNames;

	if (const UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get())
	{
		for (UPropertyAnimatorCoreTimeSourceBase* TimeSource : AnimatorSubsystem->GetTimeSources())
		{
			if (TimeSource && IsTimeSourceSupported(TimeSource))
			{
				TimeSourceNames.Add(TimeSource->GetTimeSourceName());
			}
		}
	}

	return TimeSourceNames;
}

UPropertyAnimatorCoreTimeSourceBase* UPropertyAnimatorCoreBase::FindOrAddTimeSource(FName InTimeSourceName)
{
	if (IsTemplate())
	{
		return nullptr;
	}

	UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();

	if (!Subsystem || InTimeSourceName.IsNone())
	{
		return nullptr;
	}

	// Check cached time source instances
	UPropertyAnimatorCoreTimeSourceBase* NewTimeSource = nullptr;

	for (const TObjectPtr<UPropertyAnimatorCoreTimeSourceBase>& TimeSource : TimeSources)
	{
		if (TimeSource && TimeSource->GetTimeSourceName() == InTimeSourceName)
		{
			NewTimeSource = TimeSource.Get();
		}
	}

	// Create new time source instance and cache it
	if (!NewTimeSource)
	{
		NewTimeSource = Subsystem->CreateNewTimeSource(InTimeSourceName, this);

		if (NewTimeSource)
		{
			TimeSources.Add(NewTimeSource);
		}
	}

	return NewTimeSource;
}

UPropertyAnimatorCoreTimeSourceBase* UPropertyAnimatorCoreBase::GetActiveTimeSource() const
{
	UPropertyAnimatorCoreTimeSourceBase* TimeSource = nullptr;

	if (bOverrideTimeSource)
	{
		TimeSource = ActiveTimeSource;
	}
	else if (const UPropertyAnimatorCoreComponent* AnimatorComponent = GetAnimatorComponent())
	{
		TimeSource = AnimatorComponent->GetAnimatorsActiveTimeSource();
	}

	return TimeSource;
}

void UPropertyAnimatorCoreBase::SetAnimatorDisplayName(FName InName)
{
	if (AnimatorDisplayName == InName)
	{
		return;
	}

	AnimatorDisplayName = InName;
	OnAnimatorDisplayNameChanged();

	OnAnimatorRenamedDelegate.Broadcast(GetAnimatorComponent(), this, EPropertyAnimatorCoreUpdateEvent::User);
}

TSet<FPropertyAnimatorCoreData> UPropertyAnimatorCoreBase::GetLinkedProperties() const
{
	TSet<FPropertyAnimatorCoreData> LinkedPropertiesSet;

	for (const TObjectPtr<UPropertyAnimatorCoreContext>& Options : LinkedProperties)
	{
		if (Options)
		{
			LinkedPropertiesSet.Emplace(Options->GetAnimatedProperty());
		}
	}

	return LinkedPropertiesSet;
}

int32 UPropertyAnimatorCoreBase::GetLinkedPropertiesCount() const
{
	return LinkedProperties.Num();
}

UPropertyAnimatorCoreContext* UPropertyAnimatorCoreBase::LinkProperty(const FPropertyAnimatorCoreData& InLinkProperty)
{
	UPropertyAnimatorCoreContext* PropertyContext = nullptr;

	if (!InLinkProperty.IsResolved())
	{
		return PropertyContext;
	}

	const UObject* Owner = InLinkProperty.GetOwner();
	const AActor* OwningActor = GetTypedOuter<AActor>();

	if (Owner != OwningActor && !Owner->IsIn(OwningActor))
	{
		return PropertyContext;
	}

	const EPropertyAnimatorPropertySupport Support = GetPropertySupport(InLinkProperty);

	if (Support == EPropertyAnimatorPropertySupport::None)
	{
		return PropertyContext;
	}

	if (IsPropertyLinked(InLinkProperty) || !GetInnerPropertiesLinked(InLinkProperty).IsEmpty())
	{
		return GetLinkedPropertyContext(InLinkProperty);
	}

	const TSubclassOf<UPropertyAnimatorCoreContext> ContextSubclass = GetPropertyContextClass(InLinkProperty);
	const UClass* ContextClass = ContextSubclass.Get();

	if (!IsValid(ContextClass))
	{
		return PropertyContext;
	}

	PropertyContext = NewObject<UPropertyAnimatorCoreContext>(this, ContextClass, NAME_None, RF_Transactional);
	PropertyContext->ConstructInternal(InLinkProperty);

	LinkedProperties.Add(PropertyContext);
	OnPropertyLinked(PropertyContext, Support);

	UPropertyAnimatorCoreBase::OnAnimatorPropertyLinkedDelegate.Broadcast(this, InLinkProperty);

	return PropertyContext;
}

bool UPropertyAnimatorCoreBase::UnlinkProperty(const FPropertyAnimatorCoreData& InUnlinkProperty)
{
	if (!IsPropertyLinked(InUnlinkProperty))
	{
		return false;
	}

	if (UPropertyAnimatorCoreContext* PropertyContext = GetLinkedPropertyContext(InUnlinkProperty))
	{
		PropertyContext->Restore();
		LinkedProperties.Remove(PropertyContext);
		OnPropertyUnlinked(PropertyContext);
	}

	UPropertyAnimatorCoreBase::OnAnimatorPropertyUnlinkedDelegate.Broadcast(this, InUnlinkProperty);

	return true;
}

bool UPropertyAnimatorCoreBase::IsPropertyLinked(const FPropertyAnimatorCoreData& InPropertyData) const
{
	return LinkedProperties.ContainsByPredicate([&InPropertyData](const UPropertyAnimatorCoreContext* InOptions)
	{
		return InOptions
			&& (
				InOptions->GetAnimatedProperty() == InPropertyData
				|| InOptions->GetAnimatedProperty().IsOwning(InPropertyData)
			);
	});
}

bool UPropertyAnimatorCoreBase::IsPropertiesLinked(const TSet<FPropertyAnimatorCoreData>& InProperties) const
{
	for (const FPropertyAnimatorCoreData& Property : InProperties)
	{
		if (!IsPropertyLinked(Property))
		{
			return false;
		}
	}

	return !InProperties.IsEmpty();
}

TSet<FPropertyAnimatorCoreData> UPropertyAnimatorCoreBase::GetInnerPropertiesLinked(const FPropertyAnimatorCoreData& InPropertyData) const
{
	TSet<FPropertyAnimatorCoreData> OutProperties;

	if (!InPropertyData.IsResolved())
	{
		return OutProperties;
	}

	for (const FPropertyAnimatorCoreData& LinkedProperty : GetLinkedProperties())
	{
		if (InPropertyData.IsOwning(LinkedProperty))
		{
			OutProperties.Add(LinkedProperty);
		}
	}

	return OutProperties;
}
