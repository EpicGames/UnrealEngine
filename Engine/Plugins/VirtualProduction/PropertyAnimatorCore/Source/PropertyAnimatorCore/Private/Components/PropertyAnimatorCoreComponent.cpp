// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/PropertyAnimatorCoreComponent.h"

#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Settings/PropertyAnimatorCoreSettings.h"
#include "Subsystems/PropertyAnimatorCoreSubsystem.h"
#include "TimeSources/PropertyAnimatorCoreTimeSourceBase.h"

UPropertyAnimatorCoreBase* UPropertyAnimatorCoreComponent::AddAnimator(const UClass* InAnimatorClass)
{
	if (!InAnimatorClass)
	{
		return nullptr;
	}

	UPropertyAnimatorCoreBase* NewAnimator = NewObject<UPropertyAnimatorCoreBase>(this, InAnimatorClass, NAME_None, RF_Transactional);

	if (NewAnimator)
	{
		PropertyAnimatorsInternal = PropertyAnimators;
		PropertyAnimators.Add(NewAnimator);

		OnAnimatorsChanged(EPropertyAnimatorCoreUpdateEvent::User);
	}

	return NewAnimator;
}

UPropertyAnimatorCoreBase* UPropertyAnimatorCoreComponent::CloneAnimator(UPropertyAnimatorCoreBase* InAnimator)
{
	UPropertyAnimatorCoreBase* CloneAnimator = nullptr;

	if (!InAnimator)
	{
		return CloneAnimator;
	}

	// Duplicate animator
	FObjectDuplicationParameters Parameters = InitStaticDuplicateObjectParams(InAnimator, this);
	CloneAnimator = Cast<UPropertyAnimatorCoreBase>(StaticDuplicateObjectEx(Parameters));

	// Force current state
	CloneAnimator->OnAnimatorEnabledChanged(EPropertyAnimatorCoreUpdateEvent::User);

	PropertyAnimatorsInternal = PropertyAnimators;
	PropertyAnimators.Add(CloneAnimator);

	OnAnimatorsChanged(EPropertyAnimatorCoreUpdateEvent::User);

	return CloneAnimator;
}

bool UPropertyAnimatorCoreComponent::RemoveAnimator(UPropertyAnimatorCoreBase* InAnimator)
{
	if (!PropertyAnimators.Contains(InAnimator))
	{
		return false;
	}

	PropertyAnimatorsInternal = PropertyAnimators;
	PropertyAnimators.Remove(InAnimator);

	OnAnimatorsChanged(EPropertyAnimatorCoreUpdateEvent::User);

	return true;
}

void UPropertyAnimatorCoreComponent::OnAnimatorsSetEnabled(const UWorld* InWorld, bool bInEnabled, bool bInTransact)
{
	if (GetWorld() == InWorld)
	{
#if WITH_EDITOR
		if (bInTransact)
		{
			Modify();
		}
#endif

		SetAnimatorsEnabled(bInEnabled);
	}
}

void UPropertyAnimatorCoreComponent::OnAnimatorsChanged(EPropertyAnimatorCoreUpdateEvent InType)
{
	const TSet<TObjectPtr<UPropertyAnimatorCoreBase>> AnimatorsSet(PropertyAnimators);
	const TSet<TObjectPtr<UPropertyAnimatorCoreBase>> AnimatorsInternalSet(PropertyAnimatorsInternal);

	TSet<TObjectPtr<UPropertyAnimatorCoreBase>> RemovedAnimators = AnimatorsInternalSet.Difference(AnimatorsSet);
	TSet<TObjectPtr<UPropertyAnimatorCoreBase>> AddedAnimators = AnimatorsSet.Difference(AnimatorsInternalSet);
	PropertyAnimatorsInternal.Empty();

	for (const TObjectPtr<UPropertyAnimatorCoreBase>& RemovedAnimator : RemovedAnimators)
	{
		if (RemovedAnimator)
		{
#if WITH_EDITOR
			RemovedAnimator->Modify();
#endif

			RemovedAnimator->SetAnimatorEnabled(false);
			RemovedAnimator->OnAnimatorRemoved(InType);
		}
	}
	for (const TObjectPtr<UPropertyAnimatorCoreBase>& AddedAnimator : AddedAnimators)
	{
		if (AddedAnimator)
		{
#if WITH_EDITOR
			AddedAnimator->Modify();
#endif

			AddedAnimator->SetAnimatorDisplayName(GetAnimatorName(AddedAnimator));
			AddedAnimator->OnAnimatorAdded(InType);
			AddedAnimator->SetAnimatorEnabled(true);
		}
	}

	OnAnimatorsEnabledChanged(InType);
}

void UPropertyAnimatorCoreComponent::OnAnimatorsEnabledChanged(EPropertyAnimatorCoreUpdateEvent InType)
{
	const bool bEnableAnimators = ShouldAnimate();

	for (const TObjectPtr<UPropertyAnimatorCoreBase>& Animator : PropertyAnimators)
	{
		if (!IsValid(Animator))
		{
			continue;
		}

		// When enabling all animators, if animator is disabled then skip
		if (bEnableAnimators && !Animator->GetAnimatorEnabled())
		{
			continue;
		}

		// When disabling all animators : if animator is disabled then skip
		if (!bEnableAnimators && !Animator->GetAnimatorEnabled())
		{
			continue;
		}

		Animator->OnAnimatorEnabledChanged(InType);
	}

	SetComponentTickEnabled(bEnableAnimators);
}

void UPropertyAnimatorCoreComponent::OnTimeSourceNameChanged()
{
	if (ActiveAnimatorsTimeSource)
	{
		ActiveAnimatorsTimeSource->DeactivateTimeSource();
	}

	ActiveAnimatorsTimeSource = FindOrAddTimeSource(AnimatorsTimeSourceName);

	if (ActiveAnimatorsTimeSource)
	{
		ActiveAnimatorsTimeSource->ActivateTimeSource();
	}
}

UPropertyAnimatorCoreTimeSourceBase* UPropertyAnimatorCoreComponent::FindOrAddTimeSource(FName InTimeSourceName)
{
	if (IsTemplate())
	{
		return nullptr;
	}

	UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();

	if (!Subsystem || InTimeSourceName.IsNone())
	{
		return nullptr;
	}

	// Check cached time source instances
	UPropertyAnimatorCoreTimeSourceBase* NewTimeSource = nullptr;

	for (const TObjectPtr<UPropertyAnimatorCoreTimeSourceBase>& TimeSource : TimeSources)
	{
		if (TimeSource && TimeSource->GetTimeSourceName() == InTimeSourceName)
		{
			NewTimeSource = TimeSource.Get();
		}
	}

	// Create new time source instance and cache it
	if (!NewTimeSource)
	{
		NewTimeSource = Subsystem->CreateNewTimeSource(InTimeSourceName, this);

		if (NewTimeSource)
		{
			TimeSources.Add(NewTimeSource);
		}
	}

	return NewTimeSource;
}

bool UPropertyAnimatorCoreComponent::ShouldAnimate() const
{
	return bAnimatorsEnabled
		&& !PropertyAnimators.IsEmpty()
		&& !FMath::IsNearlyZero(AnimatorsMagnitude);
}

FName UPropertyAnimatorCoreComponent::GetAnimatorName(const UPropertyAnimatorCoreBase* InAnimator)
{
	if (!InAnimator)
	{
		return NAME_None;
	}

	FString NewAnimatorName = InAnimator->GetName();

	const int32 Idx = NewAnimatorName.Find(InAnimator->GetAnimatorMetadata()->Name.ToString());
	if (Idx != INDEX_NONE)
	{
		NewAnimatorName = NewAnimatorName.RightChop(Idx);
	}

	return FName(NewAnimatorName);
}

void UPropertyAnimatorCoreComponent::OnComponentCreated()
{
	Super::OnComponentCreated();

	if (AActor* OwningActor = GetOwner())
	{
		// For spawnable templates, restore and resolve properties owner
		for (const TObjectPtr<UPropertyAnimatorCoreBase>& Animator : PropertyAnimators)
		{
			if (Animator)
			{
				Animator->RestoreProperties();
				Animator->ResolvePropertiesOwner(OwningActor);
			}
		}
	}

	UPropertyAnimatorCoreBase::OnAnimatorAddedDelegate.Broadcast(this, nullptr, EPropertyAnimatorCoreUpdateEvent::User);
}

UPropertyAnimatorCoreComponent* UPropertyAnimatorCoreComponent::FindOrAdd(AActor* InActor)
{
	if (!IsValid(InActor))
	{
		return nullptr;
	}

	if (UPropertyAnimatorCoreComponent* ExistingComponent = InActor->FindComponentByClass<UPropertyAnimatorCoreComponent>())
	{
		return ExistingComponent;
	}

#if WITH_EDITOR
	InActor->Modify();
#endif

	const UClass* const ComponentClass = UPropertyAnimatorCoreComponent::StaticClass();

	// Construct the new component and attach as needed
	UPropertyAnimatorCoreComponent* const PropertyAnimatorComponent = NewObject<UPropertyAnimatorCoreComponent>(InActor
		, ComponentClass
		, MakeUniqueObjectName(InActor, ComponentClass, TEXT("PropertyAnimatorComponent"))
		, RF_Transactional);

	// Add to SerializedComponents array so it gets saved
	InActor->AddInstanceComponent(PropertyAnimatorComponent);
	PropertyAnimatorComponent->OnComponentCreated();
	PropertyAnimatorComponent->RegisterComponent();

#if WITH_EDITOR
	// Rerun construction scripts
	InActor->RerunConstructionScripts();
#endif

	return PropertyAnimatorComponent;
}

UPropertyAnimatorCoreComponent::UPropertyAnimatorCoreComponent()
{
	if (!IsTemplate())
	{
		bTickInEditor = true;
		PrimaryComponentTick.bCanEverTick = true;

		// Used to toggle animators state in world
		UPropertyAnimatorCoreSubsystem::OnAnimatorsSetEnabledDelegate.AddUObject(this, &UPropertyAnimatorCoreComponent::OnAnimatorsSetEnabled);

		if (const UPropertyAnimatorCoreSettings* AnimatorSettings = UPropertyAnimatorCoreSettings::Get())
		{
			SetAnimatorsTimeSourceName(AnimatorSettings->GetDefaultTimeSourceName());
		}
	}
}

void UPropertyAnimatorCoreComponent::SetAnimatorsEnabled(bool bInEnabled)
{
	if (bAnimatorsEnabled == bInEnabled)
	{
		return;
	}

	bAnimatorsEnabled = bInEnabled;
	OnAnimatorsEnabledChanged(EPropertyAnimatorCoreUpdateEvent::User);
}

void UPropertyAnimatorCoreComponent::SetAnimatorsMagnitude(float InMagnitude)
{
	InMagnitude = FMath::Clamp(InMagnitude, 0.f, 1.f);

	if (FMath::IsNearlyEqual(AnimatorsMagnitude, InMagnitude))
	{
		return;
	}

	AnimatorsMagnitude = InMagnitude;
	OnAnimatorsEnabledChanged(EPropertyAnimatorCoreUpdateEvent::User);
}

void UPropertyAnimatorCoreComponent::OnComponentDestroyed(bool bInDestroyingHierarchy)
{
	Super::OnComponentDestroyed(bInDestroyingHierarchy);

	PropertyAnimatorsInternal = PropertyAnimators;
	PropertyAnimators.Empty();

	OnAnimatorsChanged(EPropertyAnimatorCoreUpdateEvent::Destroyed);

	UPropertyAnimatorCoreBase::OnAnimatorRemovedDelegate.Broadcast(this, nullptr, EPropertyAnimatorCoreUpdateEvent::Destroyed);
}

void UPropertyAnimatorCoreComponent::TickComponent(float InDeltaTime, ELevelTick InTickType, FActorComponentTickFunction* InTickFunction)
{
	Super::TickComponent(InDeltaTime, InTickType, InTickFunction);

	if (!EvaluateAnimators())
	{
		SetComponentTickEnabled(false);
	}
}

void UPropertyAnimatorCoreComponent::PostLoad()
{
	Super::PostLoad();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	// Migrate animators to new array property
	if (!Animators.IsEmpty() && PropertyAnimators.IsEmpty())
	{
		PropertyAnimators = Animators.Array();
		Animators.Empty();
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	OnTimeSourceNameChanged();
	
	UPropertyAnimatorCoreBase::OnAnimatorAddedDelegate.Broadcast(this, nullptr, EPropertyAnimatorCoreUpdateEvent::Load);
}

void UPropertyAnimatorCoreComponent::PostEditImport()
{
	Super::PostEditImport();

	OnTimeSourceNameChanged();

	UPropertyAnimatorCoreBase::OnAnimatorAddedDelegate.Broadcast(this, nullptr, EPropertyAnimatorCoreUpdateEvent::Duplicate);
}

void UPropertyAnimatorCoreComponent::PostDuplicate(EDuplicateMode::Type InMode)
{
	Super::PostDuplicate(InMode);

	OnTimeSourceNameChanged();

	UPropertyAnimatorCoreBase::OnAnimatorAddedDelegate.Broadcast(this, nullptr, EPropertyAnimatorCoreUpdateEvent::Duplicate);
}

#if WITH_EDITOR
FName UPropertyAnimatorCoreComponent::GetAnimatorsEnabledPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreComponent, bAnimatorsEnabled);
}

FName UPropertyAnimatorCoreComponent::GetPropertyAnimatorsPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreComponent, PropertyAnimators);
}

void UPropertyAnimatorCoreComponent::PreEditUndo()
{
	Super::PreEditUndo();

	PropertyAnimatorsInternal = PropertyAnimators;
}

void UPropertyAnimatorCoreComponent::PostEditUndo()
{
	Super::PostEditUndo();

	if (!bRegistered && IsValidChecked(this))
	{
		UPropertyAnimatorCoreBase::OnAnimatorAddedDelegate.Broadcast(this, nullptr, EPropertyAnimatorCoreUpdateEvent::Undo);
	}

	OnAnimatorsChanged(EPropertyAnimatorCoreUpdateEvent::Undo);
}

void UPropertyAnimatorCoreComponent::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	const FName MemberName = PropertyAboutToChange ? PropertyAboutToChange->GetFName() : NAME_None;

	if (MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreComponent, PropertyAnimators))
	{
		PropertyAnimatorsInternal = PropertyAnimators;
	}
}

void UPropertyAnimatorCoreComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreComponent, bAnimatorsEnabled)
		|| MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreComponent, AnimatorsMagnitude))
	{
		OnAnimatorsEnabledChanged(EPropertyAnimatorCoreUpdateEvent::User);
	}
	else if (MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreComponent, PropertyAnimators))
	{
		OnAnimatorsChanged(EPropertyAnimatorCoreUpdateEvent::User);
	}
	else if (MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreComponent, AnimatorsTimeSourceName))
	{
		OnTimeSourceNameChanged();
	}
}
#endif

void UPropertyAnimatorCoreComponent::SetAnimators(const TArray<TObjectPtr<UPropertyAnimatorCoreBase>>& InAnimators)
{
	PropertyAnimatorsInternal = PropertyAnimators;
	PropertyAnimators = InAnimators;
	OnAnimatorsChanged(EPropertyAnimatorCoreUpdateEvent::User);
}

void UPropertyAnimatorCoreComponent::SetAnimatorsTimeSourceName(FName InTimeSourceName)
{
	if (AnimatorsTimeSourceName == InTimeSourceName)
	{
		return;
	}

	const TArray<FName> TimeSourceNames = GetTimeSourceNames();
	if (!TimeSourceNames.Contains(InTimeSourceName))
	{
		return;
	}

	AnimatorsTimeSourceName = InTimeSourceName;
	OnTimeSourceNameChanged();
}

void UPropertyAnimatorCoreComponent::ForEachAnimator(const TFunctionRef<bool(UPropertyAnimatorCoreBase*)> InFunction) const
{
	for (const TObjectPtr<UPropertyAnimatorCoreBase>& Animator : PropertyAnimators)
	{
		if (Animator)
		{
			if (!InFunction(Animator))
			{
				break;
			}
		}
	}
}

bool UPropertyAnimatorCoreComponent::EvaluateAnimators()
{
	if (!ShouldAnimate())
	{
		return false;
	}

	if (UE::IsSavingPackage(this))
	{
		return true;
	}

	const UWorld* World = GetWorld();
	const bool bIsSupportedWorld = IsValid(World) && (World->IsGameWorld() || World->IsEditorWorld());

	if (!bIsSupportedWorld || !ActiveAnimatorsTimeSource)
	{
		return false;
	}

	FInstancedPropertyBag Parameters;

	FPropertyAnimatorCoreTimeSourceEvaluationData GlobalEvaluationData;
	EPropertyAnimatorCoreTimeSourceResult GlobalTimeResult = ActiveAnimatorsTimeSource->FetchEvaluationData(GlobalEvaluationData);

	for (const TObjectPtr<UPropertyAnimatorCoreBase>& Animator : PropertyAnimators)
	{
		if (!IsValid(Animator) || !Animator->GetAnimatorEnabled())
		{
			continue;
		}

		UPropertyAnimatorCoreTimeSourceBase* AnimatorTimeSource = Animator->GetActiveTimeSource();

		if (!AnimatorTimeSource)
		{
			continue;
		}

		FPropertyAnimatorCoreTimeSourceEvaluationData AnimatorEvaluationData = GlobalEvaluationData;
		EPropertyAnimatorCoreTimeSourceResult AnimatorTimeResult = GlobalTimeResult;

		if (Animator->GetOverrideTimeSource())
		{
			AnimatorTimeResult = AnimatorTimeSource->FetchEvaluationData(AnimatorEvaluationData);
		}

		if (AnimatorTimeResult != EPropertyAnimatorCoreTimeSourceResult::Evaluate)
		{
			if (AnimatorTimeResult == EPropertyAnimatorCoreTimeSourceResult::Idle)
			{
				Animator->OnTimeSourceEnterIdleState();
			}

			continue;
		}

		// Reset in case animator change values to avoid affecting following animators
		Parameters.Reset();

		Parameters.AddProperty(UPropertyAnimatorCoreBase::MagnitudeParameterName, EPropertyBagPropertyType::Float);
		Parameters.SetValueFloat(UPropertyAnimatorCoreBase::MagnitudeParameterName, AnimatorsMagnitude * AnimatorEvaluationData.Magnitude);

		Parameters.AddProperty(UPropertyAnimatorCoreBase::TimeElapsedParameterName, EPropertyBagPropertyType::Double);
		Parameters.SetValueDouble(UPropertyAnimatorCoreBase::TimeElapsedParameterName, AnimatorEvaluationData.TimeElapsed);

		Animator->EvaluateAnimator(Parameters);
	}

	return true;
}

TArray<FName> UPropertyAnimatorCoreComponent::GetTimeSourceNames() const
{
	TArray<FName> TimeSourceNames;

	if (const UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get())
	{
		TimeSourceNames = AnimatorSubsystem->GetTimeSourceNames();
	}

	return TimeSourceNames;
}
