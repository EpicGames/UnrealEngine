// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presets/PropertyAnimatorCoreAnimatorPreset.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Presets/PropertyAnimatorCorePresetable.h"

bool UPropertyAnimatorCoreAnimatorPreset::IsPresetApplied(const UPropertyAnimatorCoreBase* InAnimator) const
{
	return false;
}

bool UPropertyAnimatorCoreAnimatorPreset::IsPresetSupported(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator) const
{
	return InAnimator && InAnimator->IsA(TargetAnimatorClass);
}

bool UPropertyAnimatorCoreAnimatorPreset::ApplyPreset(UPropertyAnimatorCoreBase* InAnimator)
{
	if (InAnimator->IsTemplate() || !AnimatorPreset.IsValid())
	{
		return false;
	}

	return InAnimator->ImportPreset(this, AnimatorPreset.ToSharedRef());
}

bool UPropertyAnimatorCoreAnimatorPreset::UnapplyPreset(UPropertyAnimatorCoreBase* InAnimator)
{
	return false;
}

void UPropertyAnimatorCoreAnimatorPreset::CreatePreset(FName InName, const TArray<IPropertyAnimatorCorePresetable*>& InPresetableItem)
{
	Super::CreatePreset(InName, InPresetableItem);

	TSharedPtr<FPropertyAnimatorCorePresetArchive> ItemArchive;

	if (InPresetableItem[0]
		&& InPresetableItem[0]->ExportPreset(this, ItemArchive)
		&& ItemArchive.IsValid())
	{
		FString OutputString;
		if (ItemArchive->ToString(OutputString))
		{
			PresetVersion = 0;
			PresetFormat = ItemArchive->GetImplementationType();
			PresetContent = OutputString;
		}
	}
}

bool UPropertyAnimatorCoreAnimatorPreset::LoadPreset()
{
	if (PresetContent.IsEmpty())
	{
		return false;
	}

	TSharedRef<FPropertyAnimatorCorePresetObjectArchive> ObjectArchive = GetArchiveImplementation()->CreateObject();
	if (!ObjectArchive->FromString(PresetContent))
	{
		return false;
	}

	FString Class;
	ObjectArchive->Get(TEXT("AnimatorClass"), Class);

	if (UClass* AnimatorClass = LoadObject<UClass>(nullptr, *Class))
	{
		TargetAnimatorClass = AnimatorClass;
		AnimatorPreset = ObjectArchive;
		return true;
	}

	return false;
}

UPropertyAnimatorCoreBase* UPropertyAnimatorCoreAnimatorPreset::GetAnimatorTemplate() const
{
	return TargetAnimatorClass.GetDefaultObject();
}
