// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presets/PropertyAnimatorCoreJsonPresetArchive.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"

TSharedPtr<FPropertyAnimatorCorePresetJsonArchiveImplementation> FPropertyAnimatorCorePresetJsonArchiveImplementation::Instance;

TSharedPtr<FPropertyAnimatorCorePresetJsonArchiveImplementation> FPropertyAnimatorCorePresetJsonArchiveImplementation::Get()
{
	if (!Instance.IsValid())
	{
		Instance = MakeShared<FPropertyAnimatorCorePresetJsonArchiveImplementation>();
	}

	return Instance;
}

TSharedRef<FPropertyAnimatorCorePresetObjectArchive> FPropertyAnimatorCorePresetJsonArchiveImplementation::CreateObject()
{
	return MakeShared<FPropertyAnimatorCorePresetJsonObjectArchive>();
}

TSharedRef<FPropertyAnimatorCorePresetArrayArchive> FPropertyAnimatorCorePresetJsonArchiveImplementation::CreateArray()
{
	return MakeShared<FPropertyAnimatorCorePresetJsonArrayArchive>();
}

TSharedRef<FPropertyAnimatorCorePresetValueArchive> FPropertyAnimatorCorePresetJsonArchiveImplementation::CreateValue(bool bInValue)
{
	return MakeShared<FPropertyAnimatorCorePresetJsonValueArchive>(bInValue);
}

TSharedRef<FPropertyAnimatorCorePresetValueArchive> FPropertyAnimatorCorePresetJsonArchiveImplementation::CreateValue(const FString& InValue)
{
	return MakeShared<FPropertyAnimatorCorePresetJsonValueArchive>(InValue);
}

TSharedRef<FPropertyAnimatorCorePresetValueArchive> FPropertyAnimatorCorePresetJsonArchiveImplementation::CreateValue(uint64 InValue)
{
	return MakeShared<FPropertyAnimatorCorePresetJsonValueArchive>(InValue);
}

TSharedRef<FPropertyAnimatorCorePresetValueArchive> FPropertyAnimatorCorePresetJsonArchiveImplementation::CreateValue(int64 InValue)
{
	return MakeShared<FPropertyAnimatorCorePresetJsonValueArchive>(InValue);
}

TSharedRef<FPropertyAnimatorCorePresetValueArchive> FPropertyAnimatorCorePresetJsonArchiveImplementation::CreateValue(double InValue)
{
	return MakeShared<FPropertyAnimatorCorePresetJsonValueArchive>(InValue);
}

FPropertyAnimatorCorePresetJsonObjectArchive::FPropertyAnimatorCorePresetJsonObjectArchive()
{
	JsonObject = MakeShared<FJsonObject>();
}

FPropertyAnimatorCorePresetJsonObjectArchive::FPropertyAnimatorCorePresetJsonObjectArchive(TSharedRef<FJsonObject> InJsonObject)
{
	JsonObject = InJsonObject;
}

bool FPropertyAnimatorCorePresetJsonObjectArchive::Remove(const FString& InKey)
{
	if (!JsonObject->HasField(InKey))
	{
		return false;
	}

	JsonObject->RemoveField(InKey);
	return true;
}

void FPropertyAnimatorCorePresetJsonObjectArchive::Clear()
{
	JsonObject->Values.Empty();
}

bool FPropertyAnimatorCorePresetJsonObjectArchive::Set(const FString& InKey, TSharedRef<FPropertyAnimatorCorePresetArchive> InValue)
{
	check(InValue->GetImplementationType().IsEqual(GetImplementationType()))

	if (const FPropertyAnimatorCorePresetJsonArrayArchive* Array = static_cast<FPropertyAnimatorCorePresetJsonArrayArchive*>(InValue->AsMutableArray().Get()))
	{
		JsonObject->SetArrayField(InKey, Array->GetJsonValues());
	}
	else if (const FPropertyAnimatorCorePresetJsonObjectArchive* Object = static_cast<FPropertyAnimatorCorePresetJsonObjectArchive*>(InValue->AsMutableObject().Get()))
	{
		JsonObject->SetObjectField(InKey, Object->GetJsonObject());
	}
	else if (const FPropertyAnimatorCorePresetJsonValueArchive* Value = static_cast<FPropertyAnimatorCorePresetJsonValueArchive*>(InValue->AsMutableValue().Get()))
	{
		JsonObject->SetField(InKey, Value->GetJsonValue());
	}

	return true;
}

bool FPropertyAnimatorCorePresetJsonObjectArchive::Set(const FString& InKey, bool bInValue)
{
	return Set(InKey, GetImplementation()->CreateValue(bInValue));
}

bool FPropertyAnimatorCorePresetJsonObjectArchive::Set(const FString& InKey, uint64 InValue)
{
	return Set(InKey, GetImplementation()->CreateValue(InValue));
}

bool FPropertyAnimatorCorePresetJsonObjectArchive::Set(const FString& InKey, int64 InValue)
{
	return Set(InKey, GetImplementation()->CreateValue(InValue));
}

bool FPropertyAnimatorCorePresetJsonObjectArchive::Set(const FString& InKey, double InValue)
{
	return Set(InKey, GetImplementation()->CreateValue(InValue));
}

bool FPropertyAnimatorCorePresetJsonObjectArchive::Set(const FString& InKey, const FString& InValue)
{
	return Set(InKey, GetImplementation()->CreateValue(InValue));
}

bool FPropertyAnimatorCorePresetJsonObjectArchive::Get(const FString& InKey, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const
{
	TSharedPtr<FJsonValue> JsonValue = JsonObject->TryGetField(InKey);

	if (!JsonValue.IsValid())
	{
		return false;
	}

	if (JsonValue->Type == EJson::Array)
	{
		OutValue = MakeShared<FPropertyAnimatorCorePresetJsonArrayArchive>(JsonValue->AsArray());
	}
	else if (JsonValue->Type == EJson::Object)
	{
		OutValue = MakeShared<FPropertyAnimatorCorePresetJsonObjectArchive>(JsonValue->AsObject().ToSharedRef());
	}
	else if (JsonValue->Type != EJson::None)
	{
		OutValue = MakeShared<FPropertyAnimatorCorePresetJsonValueArchive>(JsonValue.ToSharedRef());
	}
	else
	{
		return false;
	}

	return true;
}

bool FPropertyAnimatorCorePresetJsonObjectArchive::Get(const FString& InKey, bool& bOutValue) const
{
	TSharedPtr<FPropertyAnimatorCorePresetArchive> Value;

	return Get(InKey, Value)
		&& Value->IsValue()
		&& Value->AsMutableValue()->Get(bOutValue);
}

bool FPropertyAnimatorCorePresetJsonObjectArchive::Get(const FString& InKey, uint64& OutValue) const
{
	TSharedPtr<FPropertyAnimatorCorePresetArchive> Value;

	return Get(InKey, Value)
		&& Value->IsValue()
		&& Value->AsMutableValue()->Get(OutValue);
}

bool FPropertyAnimatorCorePresetJsonObjectArchive::Get(const FString& InKey, int64& OutValue) const
{
	TSharedPtr<FPropertyAnimatorCorePresetArchive> Value;

	return Get(InKey, Value)
		&& Value->IsValue()
		&& Value->AsMutableValue()->Get(OutValue);
}

bool FPropertyAnimatorCorePresetJsonObjectArchive::Get(const FString& InKey, double& OutValue) const
{
	TSharedPtr<FPropertyAnimatorCorePresetArchive> Value;

	return Get(InKey, Value)
		&& Value->IsValue()
		&& Value->AsMutableValue()->Get(OutValue);
}

bool FPropertyAnimatorCorePresetJsonObjectArchive::Get(const FString& InKey, FString& OutValue) const
{
	TSharedPtr<FPropertyAnimatorCorePresetArchive> Value;

	return Get(InKey, Value)
		&& Value->IsValue()
		&& Value->AsMutableValue()->Get(OutValue);
}

bool FPropertyAnimatorCorePresetJsonObjectArchive::ToString(FString& OutString) const
{
	const TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&OutString);
	return JsonObject.IsValid() && FJsonSerializer::Serialize(JsonObject.ToSharedRef(), JsonWriter);
}

bool FPropertyAnimatorCorePresetJsonObjectArchive::FromString(const FString& InString)
{
	const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(InString);
	return FJsonSerializer::Deserialize(JsonReader, JsonObject) && JsonObject.IsValid();
}

TSharedRef<FPropertyAnimatorCorePresetArchiveImplementation> FPropertyAnimatorCorePresetJsonObjectArchive::GetImplementation() const
{
	return FPropertyAnimatorCorePresetJsonArchiveImplementation::Get().ToSharedRef();
}

FPropertyAnimatorCorePresetJsonArrayArchive::FPropertyAnimatorCorePresetJsonArrayArchive(const TArray<TSharedPtr<FJsonValue>>& InJsonArray)
{
	JsonValues = InJsonArray;
}

bool FPropertyAnimatorCorePresetJsonArrayArchive::Get(int32 InIndex, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const
{
	if (!JsonValues.IsValidIndex(InIndex) || !JsonValues[InIndex].IsValid())
	{
		return false;
	}

	if (JsonValues[InIndex]->Type == EJson::Array)
	{
		OutValue = MakeShared<FPropertyAnimatorCorePresetJsonArrayArchive>(JsonValues[InIndex]->AsArray());
	}
	else if (JsonValues[InIndex]->Type == EJson::Object)
	{
		OutValue = MakeShared<FPropertyAnimatorCorePresetJsonObjectArchive>(JsonValues[InIndex]->AsObject().ToSharedRef());
	}
	else if (JsonValues[InIndex]->Type != EJson::None)
	{
		OutValue = MakeShared<FPropertyAnimatorCorePresetJsonValueArchive>(JsonValues[InIndex].ToSharedRef());
	}
	else
	{
		return false;
	}

	return true;
}

int32 FPropertyAnimatorCorePresetJsonArrayArchive::Num() const
{
	return JsonValues.Num();
}

bool FPropertyAnimatorCorePresetJsonArrayArchive::Remove(int32 InIndex)
{
	if (!JsonValues.IsValidIndex(InIndex))
	{
		return false;
	}

	JsonValues.RemoveAt(InIndex);
	return true;
}

void FPropertyAnimatorCorePresetJsonArrayArchive::Clear()
{
	JsonValues.Empty();
}

bool FPropertyAnimatorCorePresetJsonArrayArchive::Add(TSharedRef<FPropertyAnimatorCorePresetArchive> InValue)
{
	check(InValue->GetImplementationType().IsEqual(GetImplementationType()))

	if (FPropertyAnimatorCorePresetJsonArrayArchive* Array = static_cast<FPropertyAnimatorCorePresetJsonArrayArchive*>(InValue->AsMutableArray().Get()))
	{
		JsonValues.Add(MakeShared<FJsonValueArray>(Array->GetJsonValues()));
	}
	else if (FPropertyAnimatorCorePresetJsonObjectArchive* Object = static_cast<FPropertyAnimatorCorePresetJsonObjectArchive*>(InValue->AsMutableObject().Get()))
	{
		JsonValues.Add(MakeShared<FJsonValueObject>(Object->GetJsonObject()));
	}
	else if (FPropertyAnimatorCorePresetJsonValueArchive* Value = static_cast<FPropertyAnimatorCorePresetJsonValueArchive*>(InValue->AsMutableValue().Get()))
	{
		JsonValues.Add(Value->GetJsonValue());
	}

	return true;
}

bool FPropertyAnimatorCorePresetJsonArrayArchive::Add(bool bInValue)
{
	return Add(GetImplementation()->CreateValue(bInValue));
}

bool FPropertyAnimatorCorePresetJsonArrayArchive::Add(uint64 InValue)
{
	return Add(GetImplementation()->CreateValue(InValue));
}

bool FPropertyAnimatorCorePresetJsonArrayArchive::Add(int64 InValue)
{
	return Add(GetImplementation()->CreateValue(InValue));
}

bool FPropertyAnimatorCorePresetJsonArrayArchive::Add(double InValue)
{
	return Add(GetImplementation()->CreateValue(InValue));
}

bool FPropertyAnimatorCorePresetJsonArrayArchive::Add(const FString& InValue)
{
	return Add(GetImplementation()->CreateValue(InValue));
}

bool FPropertyAnimatorCorePresetJsonArrayArchive::FromString(const FString& InString)
{
	const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(InString);
	return FJsonSerializer::Deserialize(JsonReader, JsonValues);
}

bool FPropertyAnimatorCorePresetJsonArrayArchive::ToString(FString& OutString) const
{
	const TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&OutString);
	return FJsonSerializer::Serialize(JsonValues, JsonWriter);
}

TSharedRef<FPropertyAnimatorCorePresetArchiveImplementation> FPropertyAnimatorCorePresetJsonArrayArchive::GetImplementation() const
{
	return FPropertyAnimatorCorePresetJsonArchiveImplementation::Get().ToSharedRef();
}

FPropertyAnimatorCorePresetJsonValueArchive::FPropertyAnimatorCorePresetJsonValueArchive(bool bInValue)
{
	JsonValue = MakeShared<FJsonValueBoolean>(bInValue);
}

FPropertyAnimatorCorePresetJsonValueArchive::FPropertyAnimatorCorePresetJsonValueArchive(uint64 InValue)
{
	JsonValue = MakeShared<FJsonValueNumber>(InValue);
}

FPropertyAnimatorCorePresetJsonValueArchive::FPropertyAnimatorCorePresetJsonValueArchive(int64 InValue)
{
	JsonValue = MakeShared<FJsonValueNumber>(InValue);
}

FPropertyAnimatorCorePresetJsonValueArchive::FPropertyAnimatorCorePresetJsonValueArchive(double InValue)
{
	JsonValue = MakeShared<FJsonValueNumber>(InValue);
}

FPropertyAnimatorCorePresetJsonValueArchive::FPropertyAnimatorCorePresetJsonValueArchive(const FString& InValue)
{
	JsonValue = MakeShared<FJsonValueString>(InValue);
}

FPropertyAnimatorCorePresetJsonValueArchive::FPropertyAnimatorCorePresetJsonValueArchive(TSharedRef<FJsonValue> InJsonValue)
{
	check(InJsonValue->Type != EJson::Array && InJsonValue->Type != EJson::Object && InJsonValue->Type != EJson::None)
	JsonValue = InJsonValue;
}

bool FPropertyAnimatorCorePresetJsonValueArchive::FromString(const FString& InString)
{
	return false;
}

bool FPropertyAnimatorCorePresetJsonValueArchive::ToString(FString& OutString) const
{
	return false;
}

bool FPropertyAnimatorCorePresetJsonValueArchive::Get(double& OutValue) const
{
	return JsonValue.IsValid() && JsonValue->TryGetNumber(OutValue);
}

bool FPropertyAnimatorCorePresetJsonValueArchive::Get(bool& bOutValue) const
{
	return JsonValue.IsValid() && JsonValue->TryGetBool(bOutValue);
}

bool FPropertyAnimatorCorePresetJsonValueArchive::Get(FString& OutValue) const
{
	return JsonValue.IsValid() && JsonValue->TryGetString(OutValue);
}

bool FPropertyAnimatorCorePresetJsonValueArchive::Get(uint64& OutValue) const
{
	return JsonValue.IsValid() && JsonValue->TryGetNumber(OutValue);
}

bool FPropertyAnimatorCorePresetJsonValueArchive::Get(int64& OutValue) const
{
	return JsonValue.IsValid() && JsonValue->TryGetNumber(OutValue);
}

TSharedRef<FPropertyAnimatorCorePresetArchiveImplementation> FPropertyAnimatorCorePresetJsonValueArchive::GetImplementation() const
{
	return FPropertyAnimatorCorePresetJsonArchiveImplementation::Get().ToSharedRef();
}
