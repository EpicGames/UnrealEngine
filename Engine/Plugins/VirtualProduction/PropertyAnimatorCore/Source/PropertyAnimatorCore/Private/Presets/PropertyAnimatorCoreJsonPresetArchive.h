// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Presets/PropertyAnimatorCorePresetArchive.h"
#include "Templates/SharedPointerFwd.h"

class FJsonObject;
class FJsonValue;

struct FPropertyAnimatorCorePresetJsonArchiveImplementation : FPropertyAnimatorCorePresetArchiveImplementation
{
	static inline const FName Type = TEXT("JSON");

	static TSharedPtr<FPropertyAnimatorCorePresetJsonArchiveImplementation> Get();

	//~ Begin FPropertyAnimatorCorePresetArchiveImplementation
	virtual TSharedRef<FPropertyAnimatorCorePresetObjectArchive> CreateObject() override;
	virtual TSharedRef<FPropertyAnimatorCorePresetArrayArchive> CreateArray() override;
	virtual TSharedRef<FPropertyAnimatorCorePresetValueArchive> CreateValue(bool bInValue) override;
	virtual TSharedRef<FPropertyAnimatorCorePresetValueArchive> CreateValue(const FString& InValue) override;
	virtual TSharedRef<FPropertyAnimatorCorePresetValueArchive> CreateValue(uint64 InValue) override;
	virtual TSharedRef<FPropertyAnimatorCorePresetValueArchive> CreateValue(int64 InValue) override;
	virtual TSharedRef<FPropertyAnimatorCorePresetValueArchive> CreateValue(double InValue) override;
	virtual FName GetImplementationType() const override { return Type; }
	//~ End FPropertyAnimatorCorePresetArchiveImplementation

private:
	static TSharedPtr<FPropertyAnimatorCorePresetJsonArchiveImplementation> Instance;
};

struct FPropertyAnimatorCorePresetJsonObjectArchive : FPropertyAnimatorCorePresetObjectArchive
{
	FPropertyAnimatorCorePresetJsonObjectArchive();
	explicit FPropertyAnimatorCorePresetJsonObjectArchive(TSharedRef<FJsonObject> InJsonObject);

	//~ Begin FPropertyAnimatorCorePresetObjectArchive
	virtual bool Remove(const FString& InKey) override;
	virtual void Clear() override;

	virtual bool Set(const FString& InKey, TSharedRef<FPropertyAnimatorCorePresetArchive> InValue) override;
	virtual bool Set(const FString& InKey, bool bInValue) override;
	virtual bool Set(const FString& InKey, uint64 InValue) override;
	virtual bool Set(const FString& InKey, int64 InValue) override;
	virtual bool Set(const FString& InKey, double InValue) override;
	virtual bool Set(const FString& InKey, const FString& InValue) override;

	virtual bool Get(const FString& InKey, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const override;
	virtual bool Get(const FString& InKey, bool& bOutValue) const override;
	virtual bool Get(const FString& InKey, uint64& OutValue) const override;
	virtual bool Get(const FString& InKey, int64& OutValue) const override;
	virtual bool Get(const FString& InKey, double& OutValue) const override;
	virtual bool Get(const FString& InKey, FString& OutValue) const override;

	virtual bool ToString(FString& OutString) const override;
	virtual bool FromString(const FString& InString) override;
	virtual TSharedRef<FPropertyAnimatorCorePresetArchiveImplementation> GetImplementation() const override;
	//~ End FPropertyAnimatorCorePresetObjectArchive

	TSharedPtr<FJsonObject> GetJsonObject() const
	{
		return JsonObject;
	}

protected:
	TSharedPtr<FJsonObject> JsonObject;
};

struct FPropertyAnimatorCorePresetJsonArrayArchive : FPropertyAnimatorCorePresetArrayArchive
{
	FPropertyAnimatorCorePresetJsonArrayArchive() = default;
	explicit FPropertyAnimatorCorePresetJsonArrayArchive(const TArray<TSharedPtr<FJsonValue>>& InJsonArray);

	//~ Begin FPropertyAnimatorCorePresetArrayArchive
	virtual bool Get(int32 InIndex, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const override;
	virtual int32 Num() const override;
	virtual bool Remove(int32 InIndex) override;
	virtual void Clear() override;

	virtual bool Add(TSharedRef<FPropertyAnimatorCorePresetArchive> InValue) override;
	virtual bool Add(bool bInValue) override;
	virtual bool Add(uint64 InValue) override;
	virtual bool Add(int64 InValue) override;
	virtual bool Add(double InValue) override;
	virtual bool Add(const FString& InValue) override;

	virtual bool ToString(FString& OutString) const override;
	virtual bool FromString(const FString& InString) override;
	virtual TSharedRef<FPropertyAnimatorCorePresetArchiveImplementation> GetImplementation() const override;
	//~ End FPropertyAnimatorCorePresetArrayArchive

	const TArray<TSharedPtr<FJsonValue>>& GetJsonValues() const
	{
		return JsonValues;
	}

protected:
	TArray<TSharedPtr<FJsonValue>> JsonValues;
};

struct FPropertyAnimatorCorePresetJsonValueArchive : FPropertyAnimatorCorePresetValueArchive
{
	explicit FPropertyAnimatorCorePresetJsonValueArchive(bool bInValue);
	explicit FPropertyAnimatorCorePresetJsonValueArchive(uint64 InValue);
	explicit FPropertyAnimatorCorePresetJsonValueArchive(int64 InValue);
	explicit FPropertyAnimatorCorePresetJsonValueArchive(double InValue);
	explicit FPropertyAnimatorCorePresetJsonValueArchive(const FString& InValue);
	explicit FPropertyAnimatorCorePresetJsonValueArchive(TSharedRef<FJsonValue> InJsonValue);

	//~ Begin FPropertyAnimatorCorePresetValueArchive
	virtual bool Get(double& OutValue) const override;
	virtual bool Get(bool& bOutValue) const override;
	virtual bool Get(FString& OutValue) const override;
	virtual bool Get(uint64& OutValue) const override;
	virtual bool Get(int64& OutValue) const override;
	virtual bool ToString(FString& OutString) const override;
	virtual bool FromString(const FString& InString) override;
	virtual TSharedRef<FPropertyAnimatorCorePresetArchiveImplementation> GetImplementation() const override;
	//~ End FPropertyAnimatorCorePresetValueArchive

	TSharedPtr<FJsonValue> GetJsonValue() const
	{
		return JsonValue;
	}

protected:
	TSharedPtr<FJsonValue> JsonValue;
};