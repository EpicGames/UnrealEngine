// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presets/PropertyAnimatorCorePresetArchive.h"

bool FPropertyAnimatorCorePresetArchive::IsObject() const
{
	return !!AsObject();
}

bool FPropertyAnimatorCorePresetArchive::IsArray() const
{
	return !!AsArray();
}

bool FPropertyAnimatorCorePresetArchive::IsValue() const
{
	return !!AsValue();
}

EPropertyAnimatorCorePresetArchiveType FPropertyAnimatorCorePresetArchive::GetType() const
{
	if (IsObject())
	{
		return EPropertyAnimatorCorePresetArchiveType::Object;
	}

	if (IsArray())
	{
		return EPropertyAnimatorCorePresetArchiveType::Array;
	}

	if (IsValue())
	{
		return EPropertyAnimatorCorePresetArchiveType::Value;
	}

	check(false);
	return EPropertyAnimatorCorePresetArchiveType::Value;
}

FName FPropertyAnimatorCorePresetArchive::GetImplementationType() const
{
	return GetImplementation()->GetImplementationType();
}

bool FPropertyAnimatorCorePresetObjectArchive::Has(const FString& InKey, TOptional<EPropertyAnimatorCorePresetArchiveType> InType) const
{
	TSharedPtr<FPropertyAnimatorCorePresetArchive> Value;
	return Get(InKey, Value) && (!InType.IsSet() || Value->GetType() == InType.GetValue());
}

TSharedPtr<const FPropertyAnimatorCorePresetObjectArchive> FPropertyAnimatorCorePresetObjectArchive::AsObject() const
{
	return SharedThis(this);
}

TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> FPropertyAnimatorCorePresetObjectArchive::AsMutableObject()
{
	return SharedThis(this);
}

TSharedPtr<const FPropertyAnimatorCorePresetArrayArchive> FPropertyAnimatorCorePresetArrayArchive::AsArray() const
{
	return SharedThis(this);
}

TSharedPtr<FPropertyAnimatorCorePresetArrayArchive> FPropertyAnimatorCorePresetArrayArchive::AsMutableArray()
{
	return SharedThis(this);
}

TSharedPtr<const FPropertyAnimatorCorePresetValueArchive> FPropertyAnimatorCorePresetValueArchive::AsValue() const
{
	return SharedThis(this);
}

TSharedPtr<FPropertyAnimatorCorePresetValueArchive> FPropertyAnimatorCorePresetValueArchive::AsMutableValue()
{
	return SharedThis(this);
}
