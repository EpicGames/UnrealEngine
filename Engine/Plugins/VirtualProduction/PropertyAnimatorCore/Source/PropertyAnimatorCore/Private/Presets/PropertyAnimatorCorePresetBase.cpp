// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presets/PropertyAnimatorCorePresetBase.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Presets/PropertyAnimatorCoreJsonPresetArchive.h"
#include "Presets/PropertyAnimatorCorePresetable.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"

FText UPropertyAnimatorCorePresetBase::GetPresetDisplayName() const
{
	return !PresetDisplayName.IsEmpty() ? PresetDisplayName : FText::FromString(FName::NameToDisplayString(PresetName.ToString(), /** IsBool */false));
}

void UPropertyAnimatorCorePresetBase::CreatePreset(FName InName, const TArray<IPropertyAnimatorCorePresetable*>& InPresetableItem)
{
	PresetName = InName;
}

TSharedRef<FPropertyAnimatorCorePresetArchiveImplementation> UPropertyAnimatorCorePresetBase::GetArchiveImplementation() const
{
	return FPropertyAnimatorCorePresetJsonArchiveImplementation::Get().ToSharedRef();
}
