// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presets/PropertyAnimatorCorePropertyPreset.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Logs/PropertyAnimatorCoreLog.h"
#include "Properties/PropertyAnimatorCoreContext.h"

void UPropertyAnimatorCorePropertyPreset::GetPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const
{
	for (const TPair<FString, TSharedRef<FPropertyAnimatorCorePresetArchive>>& PropertyPreset : PropertyPresets)
	{
		FPropertyAnimatorCoreData Property(const_cast<AActor*>(InActor), PropertyPreset.Key);

		if (Property.IsResolved())
		{
			OutProperties.Add(Property);
		}
	}
}

void UPropertyAnimatorCorePropertyPreset::OnPresetApplied(UPropertyAnimatorCoreBase* InAnimator, const TSet<FPropertyAnimatorCoreData>& InProperties)
{
	if (PropertyPresets.IsEmpty())
	{
		UE_LOGF(LogPropertyAnimatorCore, Log, "Preset %ls applied on %ls animator with %i properties", *GetPresetName().ToString(), *InAnimator->GetAnimatorMetadata()->Name.ToString(), InProperties.Num())
		return;
	}
	
	for (const TPair<FString, TSharedRef<FPropertyAnimatorCorePresetArchive>>& PropertyPreset : PropertyPresets)
	{
		FPropertyAnimatorCoreData Property(InAnimator->GetAnimatorActor(), PropertyPreset.Key);

		if (!Property.IsResolved())
		{
			continue;
		}

		bool bFound = InProperties.Contains(Property);

		if (!bFound)
		{
			for (const FPropertyAnimatorCoreData& LinkedProperty : InProperties)
			{
				if (LinkedProperty.IsChildOf(Property))
				{
					Property = LinkedProperty;
					bFound = true;
					break;
				}
			}
		}

		if (bFound)
		{
			if (UPropertyAnimatorCoreContext* Context = InAnimator->GetLinkedPropertyContext(Property))
			{
				if (Context->ImportPreset(this, PropertyPreset.Value))
				{
					UE_LOGF(LogPropertyAnimatorCore, Log, "Successfully imported preset %ls on %ls animator for property %ls", *GetPresetName().ToString(), *InAnimator->GetAnimatorMetadata()->Name.ToString(), *Property.GetPropertyDisplayName())
				}
				else
				{
					UE_LOGF(LogPropertyAnimatorCore, Warning, "Failed to import preset %ls on %ls animator for property %ls", *GetPresetName().ToString(), *InAnimator->GetAnimatorMetadata()->Name.ToString(), *Property.GetPropertyDisplayName())
				}
			}
		}
	}
}

void UPropertyAnimatorCorePropertyPreset::GetSupportedPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const
{
	TSet<FPropertyAnimatorCoreData> PresetProperties;
	GetPresetProperties(InActor, InAnimator, PresetProperties);
	OutProperties.Empty(PresetProperties.Num());

	if (PresetProperties.IsEmpty())
	{
		return;
	}

	for (const FPropertyAnimatorCoreData& PresetProperty : PresetProperties)
	{
		InAnimator->GetPropertiesSupported(PresetProperty, OutProperties, /** SearchDepth */3);
	}
}

bool UPropertyAnimatorCorePropertyPreset::IsPresetSupported(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator) const
{
	if (!IsValid(InActor) || !IsValid(InAnimator))
	{
		return false;
	}

	TSet<FPropertyAnimatorCoreData> SupportedProperties;
	GetSupportedPresetProperties(InActor, InAnimator, SupportedProperties);

	return !SupportedProperties.IsEmpty();
}

bool UPropertyAnimatorCorePropertyPreset::ApplyPreset(UPropertyAnimatorCoreBase* InAnimator)
{
	if (!IsValid(InAnimator) || InAnimator->IsTemplate())
	{
		return false;
	}

	TSet<FPropertyAnimatorCoreData> SupportedProperties;
	GetSupportedPresetProperties(InAnimator->GetAnimatorActor(), InAnimator, SupportedProperties);

	if (SupportedProperties.IsEmpty())
	{
		return false;
	}

	for (TSet<FPropertyAnimatorCoreData>::TIterator It(SupportedProperties); It; ++It)
	{
		if (!InAnimator->LinkProperty(*It))
		{
			It.RemoveCurrent();
		}
	}

	InAnimator->SetAnimatorDisplayName(FName(InAnimator->GetAnimatorMetadata()->DisplayName.ToString() + TEXT("_") + GetPresetDisplayName().ToString()));

	OnPresetApplied(InAnimator, SupportedProperties);

	return true;
}

bool UPropertyAnimatorCorePropertyPreset::IsPresetApplied(const UPropertyAnimatorCoreBase* InAnimator) const
{
	if (!IsValid(InAnimator) || InAnimator->IsTemplate())
	{
		return false;
	}

	TSet<FPropertyAnimatorCoreData> SupportedProperties;
	GetSupportedPresetProperties(InAnimator->GetAnimatorActor(), InAnimator, SupportedProperties);

	if (SupportedProperties.IsEmpty())
	{
		return false;
	}

	return InAnimator->IsPropertiesLinked(SupportedProperties);
}

bool UPropertyAnimatorCorePropertyPreset::UnapplyPreset(UPropertyAnimatorCoreBase* InAnimator)
{
	if (!IsValid(InAnimator) || InAnimator->IsTemplate())
	{
		return false;
	}

	TSet<FPropertyAnimatorCoreData> SupportedProperties;
	GetSupportedPresetProperties(InAnimator->GetAnimatorActor(), InAnimator, SupportedProperties);

	if (SupportedProperties.IsEmpty())
	{
		return false;
	}

	for (FPropertyAnimatorCoreData& SupportedProperty : SupportedProperties)
	{
		InAnimator->UnlinkProperty(SupportedProperty);
	}

	OnPresetUnapplied(InAnimator, SupportedProperties);

	return true;
}

void UPropertyAnimatorCorePropertyPreset::CreatePreset(FName InName, const TArray<IPropertyAnimatorCorePresetable*>& InPresetableItems)
{
	Super::CreatePreset(InName, InPresetableItems);

	TSharedPtr<FPropertyAnimatorCorePresetArrayArchive> PropertiesArchive = GetArchiveImplementation()->CreateArray();

	for (IPropertyAnimatorCorePresetable* InPresetableItem : InPresetableItems)
	{
		TSharedPtr<FPropertyAnimatorCorePresetArchive> PropertyArchive;

		if (InPresetableItem
			&& InPresetableItem->ExportPreset(this, PropertyArchive)
			&& PropertyArchive.IsValid())
		{
			PropertiesArchive->Add(PropertyArchive.ToSharedRef());
		}
	}

	FString OutputString;
	if (PropertiesArchive->ToString(OutputString))
	{
		PresetVersion = 0;
		PresetFormat = PropertiesArchive->GetImplementationType();
		PresetContent = OutputString;
	}
}

bool UPropertyAnimatorCorePropertyPreset::LoadPreset()
{
	if (PresetContent.IsEmpty())
	{
		return false;
	}

	TSharedPtr<FPropertyAnimatorCorePresetArrayArchive> PropertiesArchive = GetArchiveImplementation()->CreateArray();

	if (!PropertiesArchive->FromString(PresetContent) || PropertiesArchive->Num() == 0)
	{
		return false;
	}

	for (int32 Index = 0; Index < PropertiesArchive->Num(); Index++)
	{
		TSharedPtr<FPropertyAnimatorCorePresetArchive> PropertyArchive;
		if (!PropertiesArchive->Get(Index, PropertyArchive) || !PropertyArchive->IsObject())
		{
			continue;
		}

		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> PropertyObject = PropertyArchive->AsMutableObject();

		FString PropertyPath;
		if (PropertyObject->Get(UPropertyAnimatorCoreContext::GetAnimatedPropertyName().ToString(), PropertyPath) && !PropertyPath.IsEmpty())
		{
			PropertyPresets.Add(PropertyPath, PropertyArchive.ToSharedRef());
		}
	}

	return !PropertyPresets.IsEmpty();
}

void UPropertyAnimatorCorePropertyPreset::GetAppliedPresetProperties(const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutSupportedProperties, TSet<FPropertyAnimatorCoreData>& OutAppliedProperties)
{
	OutSupportedProperties.Empty();
	OutAppliedProperties.Empty();

	if (!IsValid(InAnimator) || InAnimator->IsTemplate())
	{
		return;
	}

	GetSupportedPresetProperties(InAnimator->GetAnimatorActor(), InAnimator, OutSupportedProperties);
	OutAppliedProperties.Reserve(OutSupportedProperties.Num());

	for (const FPropertyAnimatorCoreData& Property : OutSupportedProperties)
	{
		if (InAnimator->IsPropertyLinked(Property))
		{
			OutAppliedProperties.Add(Property);
		}
	}
}
