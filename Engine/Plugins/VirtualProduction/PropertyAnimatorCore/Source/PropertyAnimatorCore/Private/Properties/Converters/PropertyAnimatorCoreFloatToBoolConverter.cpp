// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/Converters/PropertyAnimatorCoreFloatToBoolConverter.h"

#include "Properties/Converters/PropertyAnimatorCoreConverterTraits.h"

bool UPropertyAnimatorCoreFloatToBoolConverter::IsConversionSupported(const FPropertyBagPropertyDesc& InFromProperty, const FPropertyBagPropertyDesc& InToProperty) const
{
	if (InFromProperty.ValueType == EPropertyBagPropertyType::Float
		&& InToProperty.ValueType == EPropertyBagPropertyType::Bool)
	{
		return true;
	}

	return Super::IsConversionSupported(InFromProperty, InToProperty);
}

bool UPropertyAnimatorCoreFloatToBoolConverter::Convert(const FPropertyBagPropertyDesc& InFromProperty, const FInstancedPropertyBag& InFromBag, const FPropertyBagPropertyDesc& InToProperty, FInstancedPropertyBag& InToBag, const TInstancedStruct<FPropertyAnimatorCoreConverterRuleBase>& InRule)
{
	TValueOrError<float, EPropertyBagResult> FromValueResult = InFromBag.GetValueFloat(InFromProperty.Name);

	if (!FromValueResult.HasValue())
	{
		return false;
	}

	bool bOutValue;
	if (TValueConverterTraits<float, bool, FBoolConverterRule>::Convert(FromValueResult.GetValue(), bOutValue, InRule.GetPtr<FBoolConverterRule>()))
	{
		InToBag.AddProperty(InToProperty.Name, EPropertyBagPropertyType::Bool);
		InToBag.SetValueBool(InToProperty.Name, bOutValue);
		return true;
	}

	return Super::Convert(InFromProperty, InFromBag, InToProperty, InToBag, InRule);
}

UScriptStruct* UPropertyAnimatorCoreFloatToBoolConverter::GetConversionRuleStruct() const
{
	return TValueConverterTraits<float, bool, FBoolConverterRule>::GetRuleStruct();
}