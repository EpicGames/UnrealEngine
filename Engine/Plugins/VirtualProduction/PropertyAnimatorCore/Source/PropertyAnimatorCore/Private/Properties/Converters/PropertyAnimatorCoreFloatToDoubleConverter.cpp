// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/Converters/PropertyAnimatorCoreFloatToDoubleConverter.h"

#include "Properties/Converters/PropertyAnimatorCoreConverterTraits.h"

bool UPropertyAnimatorCoreFloatToDoubleConverter::IsConversionSupported(const FPropertyBagPropertyDesc& InFromProperty, const FPropertyBagPropertyDesc& InToProperty) const
{
	if (InFromProperty.ValueType == EPropertyBagPropertyType::Float
		&& InToProperty.ValueType == EPropertyBagPropertyType::Double)
	{
		return true;
	}

	return Super::IsConversionSupported(InFromProperty, InToProperty);
}

bool UPropertyAnimatorCoreFloatToDoubleConverter::Convert(const FPropertyBagPropertyDesc& InFromProperty, const FInstancedPropertyBag& InFromBag, const FPropertyBagPropertyDesc& InToProperty, FInstancedPropertyBag& InToBag, const TInstancedStruct<FPropertyAnimatorCoreConverterRuleBase>& InRule)
{
	TValueOrError<float, EPropertyBagResult> FromValueResult = InFromBag.GetValueFloat(InFromProperty.Name);

	if (!FromValueResult.HasValue())
	{
		return false;
	}

	double OutValue;
	if (TValueConverterTraits<float, double>::Convert(FromValueResult.GetValue(), OutValue, nullptr))
	{
		InToBag.AddProperty(InToProperty.Name, EPropertyBagPropertyType::Double);
		InToBag.SetValueDouble(InToProperty.Name, OutValue);
		return true;
	}

	return Super::Convert(InFromProperty, InFromBag, InToProperty, InToBag, InRule);
}

UScriptStruct* UPropertyAnimatorCoreFloatToDoubleConverter::GetConversionRuleStruct() const
{
	return TValueConverterTraits<float, double>::GetRuleStruct();
}