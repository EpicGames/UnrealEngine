// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Properties/Converters/PropertyAnimatorCoreConverterBase.h"
#include "PropertyAnimatorCoreFloatToDoubleConverter.generated.h"

UCLASS(Transient)
class UPropertyAnimatorCoreFloatToDoubleConverter : public UPropertyAnimatorCoreConverterBase
{
	GENERATED_BODY()

public:
	//~ Begin UPropertyAnimatorCoreConverterBase
	virtual bool IsConversionSupported(const FPropertyBagPropertyDesc& InFromProperty, const FPropertyBagPropertyDesc& InToProperty) const override;
	virtual bool Convert(const FPropertyBagPropertyDesc& InFromProperty, const FInstancedPropertyBag& InFromBag, const FPropertyBagPropertyDesc& InToProperty, FInstancedPropertyBag& InToBag, const TInstancedStruct<FPropertyAnimatorCoreConverterRuleBase>& InRule) override;
	virtual UScriptStruct* GetConversionRuleStruct() const override;
	//~ End UPropertyAnimatorCoreConverterBase
};