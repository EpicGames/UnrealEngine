// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/Converters/PropertyAnimatorCoreFloatToIntConverter.h"

#include "Properties/Converters/PropertyAnimatorCoreConverterTraits.h"

bool UPropertyAnimatorCoreFloatToIntConverter::IsConversionSupported(const FPropertyBagPropertyDesc& InFromProperty, const FPropertyBagPropertyDesc& InToProperty) const
{
	if (InFromProperty.ValueType == EPropertyBagPropertyType::Float
		&& (InToProperty.ValueType == EPropertyBagPropertyType::Int32
			|| InToProperty.ValueType == EPropertyBagPropertyType::Byte))
	{
		return true;
	}

	return Super::IsConversionSupported(InFromProperty, InToProperty);
}

bool UPropertyAnimatorCoreFloatToIntConverter::Convert(const FPropertyBagPropertyDesc& InFromProperty, const FInstancedPropertyBag& InFromBag, const FPropertyBagPropertyDesc& InToProperty, FInstancedPropertyBag& InToBag, const TInstancedStruct<FPropertyAnimatorCoreConverterRuleBase>& InRule)
{
	TValueOrError<float, EPropertyBagResult> FromValueResult = InFromBag.GetValueFloat(InFromProperty.Name);

	if (!FromValueResult.HasValue())
	{
		return false;
	}

	int32 OutValue;
	if (TValueConverterTraits<float, int32, FInt32ConverterRule>::Convert(FromValueResult.GetValue(), OutValue, InRule.GetPtr<FInt32ConverterRule>()))
	{
		InToBag.AddProperty(InToProperty.Name, EPropertyBagPropertyType::Int32);
		InToBag.SetValueDouble(InToProperty.Name, OutValue);
		return true;
	}

	return Super::Convert(InFromProperty, InFromBag, InToProperty, InToBag, InRule);
}

UScriptStruct* UPropertyAnimatorCoreFloatToIntConverter::GetConversionRuleStruct() const
{
	return TValueConverterTraits<float, int32, FInt32ConverterRule>::GetRuleStruct();
}