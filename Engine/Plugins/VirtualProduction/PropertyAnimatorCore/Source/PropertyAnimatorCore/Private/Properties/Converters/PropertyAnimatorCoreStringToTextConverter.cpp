// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/Converters/PropertyAnimatorCoreStringToTextConverter.h"

#include "Properties/Converters/PropertyAnimatorCoreConverterTraits.h"

bool UPropertyAnimatorCoreStringToTextConverter::IsConversionSupported(const FPropertyBagPropertyDesc& InFromProperty, const FPropertyBagPropertyDesc& InToProperty) const
{
	if (InFromProperty.ValueType == EPropertyBagPropertyType::String
		&& InToProperty.ValueType == EPropertyBagPropertyType::Text)
	{
		return true;
	}

	return Super::IsConversionSupported(InFromProperty, InToProperty);
}

bool UPropertyAnimatorCoreStringToTextConverter::Convert(const FPropertyBagPropertyDesc& InFromProperty, const FInstancedPropertyBag& InFromBag, const FPropertyBagPropertyDesc& InToProperty, FInstancedPropertyBag& InToBag, const TInstancedStruct<FPropertyAnimatorCoreConverterRuleBase>& InRule)
{
	TValueOrError<FString, EPropertyBagResult> FromValueResult = InFromBag.GetValueString(InFromProperty.Name);

	if (!FromValueResult.HasValue())
	{
		return false;
	}

	FText OutValue;
	if (TValueConverterTraits<FString, FText, void>::Convert(FromValueResult.GetValue(), OutValue, nullptr))
	{
		InToBag.AddProperty(InToProperty.Name, EPropertyBagPropertyType::Text);
		InToBag.SetValueText(InToProperty.Name, OutValue);
		return true;
	}

	return Super::Convert(InFromProperty, InFromBag, InToProperty, InToBag, InRule);
}

UScriptStruct* UPropertyAnimatorCoreStringToTextConverter::GetConversionRuleStruct() const
{
	return TValueConverterTraits<FString, FText, void>::GetRuleStruct();
}