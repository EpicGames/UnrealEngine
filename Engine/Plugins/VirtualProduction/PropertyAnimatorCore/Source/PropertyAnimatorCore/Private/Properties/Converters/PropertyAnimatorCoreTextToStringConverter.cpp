// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/Converters/PropertyAnimatorCoreTextToStringConverter.h"

#include "Properties/Converters/PropertyAnimatorCoreConverterTraits.h"

bool UPropertyAnimatorCoreTextToStringConverter::IsConversionSupported(const FPropertyBagPropertyDesc& InFromProperty, const FPropertyBagPropertyDesc& InToProperty) const
{
	if (InFromProperty.ValueType == EPropertyBagPropertyType::Text
		&& InToProperty.ValueType == EPropertyBagPropertyType::String)
	{
		return true;
	}

	return Super::IsConversionSupported(InFromProperty, InToProperty);
}

bool UPropertyAnimatorCoreTextToStringConverter::Convert(const FPropertyBagPropertyDesc& InFromProperty, const FInstancedPropertyBag& InFromBag, const FPropertyBagPropertyDesc& InToProperty, FInstancedPropertyBag& InToBag, const TInstancedStruct<FPropertyAnimatorCoreConverterRuleBase>& InRule)
{
	TValueOrError<FText, EPropertyBagResult> FromValueResult = InFromBag.GetValueText(InFromProperty.Name);

	if (!FromValueResult.HasValue())
	{
		return false;
	}

	FString OutValue;
	if (TValueConverterTraits<FText, FString>::Convert(FromValueResult.GetValue(), OutValue, nullptr))
	{
		InToBag.AddProperty(InToProperty.Name, EPropertyBagPropertyType::String);
		InToBag.SetValueString(InToProperty.Name, OutValue);
		return true;
	}

	return Super::Convert(InFromProperty, InFromBag, InToProperty, InToBag, InRule);
}

UScriptStruct* UPropertyAnimatorCoreTextToStringConverter::GetConversionRuleStruct() const
{
	return TValueConverterTraits<FText, FString>::GetRuleStruct();
}