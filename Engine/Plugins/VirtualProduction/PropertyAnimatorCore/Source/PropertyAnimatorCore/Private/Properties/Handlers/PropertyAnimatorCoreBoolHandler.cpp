// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/Handlers/PropertyAnimatorCoreBoolHandler.h"

bool UPropertyAnimatorCoreBoolHandler::IsPropertySupported(const FPropertyAnimatorCoreData& InPropertyData) const
{
	if (InPropertyData.IsA<FBoolProperty>())
	{
		return true;
	}

	return Super::IsPropertySupported(InPropertyData);
}

bool UPropertyAnimatorCoreBoolHandler::GetValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	OutValue.AddProperty(PropertyHash, EPropertyBagPropertyType::Bool);

	bool bValue;
	InPropertyData.GetPropertyValuePtr(&bValue);

	OutValue.SetValueBool(PropertyHash, bValue);

	return true;
}

bool UPropertyAnimatorCoreBoolHandler::SetValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	TValueOrError<bool, EPropertyBagResult> ValueResult = InValue.GetValueBool(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	bool& bNewValue = ValueResult.GetValue();
	InPropertyData.SetPropertyValuePtr(&bNewValue);

	return true;
}

bool UPropertyAnimatorCoreBoolHandler::GetDefaultValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	OutValue.AddProperty(PropertyHash, EPropertyBagPropertyType::Bool);
	OutValue.SetValueBool(PropertyHash, false);
	return true;
}
