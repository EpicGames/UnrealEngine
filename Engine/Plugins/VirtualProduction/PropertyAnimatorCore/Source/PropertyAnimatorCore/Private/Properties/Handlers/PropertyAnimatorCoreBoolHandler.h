// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Properties/Handlers/PropertyAnimatorCoreHandlerBase.h"
#include "PropertyAnimatorCoreBoolHandler.generated.h"

/** Handles any boolean property values, get and set handler */
UCLASS(Transient)
class UPropertyAnimatorCoreBoolHandler : public UPropertyAnimatorCoreHandlerBase
{
	GENERATED_BODY()

public:
	//~ Begin UPropertyAnimatorCoreHandlerBase
	virtual bool IsPropertySupported(const FPropertyAnimatorCoreData& InPropertyData) const override;
	virtual bool GetValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue) override;
	virtual bool SetValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue) override;
	virtual bool GetDefaultValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue) override;
	//~ End UPropertyAnimatorCoreHandlerBase
};