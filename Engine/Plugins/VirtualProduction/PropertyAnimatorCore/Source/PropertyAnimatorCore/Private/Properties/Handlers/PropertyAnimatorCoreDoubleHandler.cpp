// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/Handlers/PropertyAnimatorCoreDoubleHandler.h"

bool UPropertyAnimatorCoreDoubleHandler::IsPropertySupported(const FPropertyAnimatorCoreData& InPropertyData) const
{
	if (InPropertyData.GetMemberPropertyTypeName() == NAME_Rotator)
	{
		return false;	
	}
	
	if (InPropertyData.IsA<FDoubleProperty>())
	{
		return true;
	}

	return Super::IsPropertySupported(InPropertyData);
}

bool UPropertyAnimatorCoreDoubleHandler::GetValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	OutValue.AddProperty(PropertyHash, EPropertyBagPropertyType::Double);

	double Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	OutValue.SetValueDouble(PropertyHash, Value);

	return true;
}

bool UPropertyAnimatorCoreDoubleHandler::SetValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	TValueOrError<double, EPropertyBagResult> ValueResult = InValue.GetValueDouble(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	double& NewValue = ValueResult.GetValue();
	InPropertyData.SetPropertyValuePtr(&NewValue);

	return true;
}

bool UPropertyAnimatorCoreDoubleHandler::IsAdditiveSupported() const
{
	return true;
}

bool UPropertyAnimatorCoreDoubleHandler::AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<double, EPropertyBagResult> ValueResult = InValue.GetValueDouble(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	double Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	double NewValue = Value + ValueResult.GetValue();
	InPropertyData.SetPropertyValuePtr(&NewValue);

	return true;
}

bool UPropertyAnimatorCoreDoubleHandler::AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValueA, const FInstancedPropertyBag& InValueB, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<double, EPropertyBagResult> ValueAResult = InValueA.GetValueDouble(PropertyHash);
	const TValueOrError<double, EPropertyBagResult> ValueBResult = InValueB.GetValueDouble(PropertyHash);
	if (!ValueAResult.HasValue() || !ValueBResult.HasValue())
	{
		return false;
	}

	const double NewValue = ValueAResult.GetValue() + ValueBResult.GetValue();
	const EPropertyBagResult Result = OutValue.SetValueDouble(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreDoubleHandler::AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<double, EPropertyBagResult> ValueResult = InValue.GetValueDouble(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	double Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	double NewValue = Value + ValueResult.GetValue();
	const EPropertyBagResult Result = OutValue.SetValueDouble(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreDoubleHandler::SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValueA, const FInstancedPropertyBag& InValueB, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<double, EPropertyBagResult> ValueAResult = InValueA.GetValueDouble(PropertyHash);
	const TValueOrError<double, EPropertyBagResult> ValueBResult = InValueB.GetValueDouble(PropertyHash);
	if (!ValueAResult.HasValue() || !ValueBResult.HasValue())
	{
		return false;
	}

	const double NewValue = ValueAResult.GetValue() - ValueBResult.GetValue();
	const EPropertyBagResult Result = OutValue.SetValueDouble(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreDoubleHandler::SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<double, EPropertyBagResult> ValueResult = InValue.GetValueDouble(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	double Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	double NewValue = Value - ValueResult.GetValue();
	InPropertyData.SetPropertyValuePtr(&NewValue);

	return true;
}

bool UPropertyAnimatorCoreDoubleHandler::SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<double, EPropertyBagResult> ValueResult = InValue.GetValueDouble(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	double Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	const double NewValue = Value - ValueResult.GetValue();
	const EPropertyBagResult Result = OutValue.SetValueDouble(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreDoubleHandler::GetDefaultValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	OutValue.AddProperty(PropertyHash, EPropertyBagPropertyType::Double);
	OutValue.SetValueDouble(PropertyHash, 0.0);
	return true;
}
