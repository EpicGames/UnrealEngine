// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/Handlers/PropertyAnimatorCoreFloatHandler.h"

bool UPropertyAnimatorCoreFloatHandler::IsPropertySupported(const FPropertyAnimatorCoreData& InPropertyData) const
{
	if (InPropertyData.GetMemberPropertyTypeName() == NAME_Rotator)
	{
		return false;	
	}
	
	if (InPropertyData.IsA<FFloatProperty>())
	{
		return true;
	}

	return Super::IsPropertySupported(InPropertyData);
}

bool UPropertyAnimatorCoreFloatHandler::GetValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	OutValue.AddProperty(PropertyHash, EPropertyBagPropertyType::Float);

	float Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	OutValue.SetValueFloat(PropertyHash, Value);

	return true;
}

bool UPropertyAnimatorCoreFloatHandler::SetValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	TValueOrError<float, EPropertyBagResult> ValueResult = InValue.GetValueFloat(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	float& NewValue = ValueResult.GetValue();
	InPropertyData.SetPropertyValuePtr(&NewValue);

	return true;
}

bool UPropertyAnimatorCoreFloatHandler::IsAdditiveSupported() const
{
	return true;
}

bool UPropertyAnimatorCoreFloatHandler::AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<float, EPropertyBagResult> ValueResult = InValue.GetValueFloat(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	float Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	float NewValue = Value + ValueResult.GetValue();
	InPropertyData.SetPropertyValuePtr(&NewValue);

	return true;
}

bool UPropertyAnimatorCoreFloatHandler::AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValueA, const FInstancedPropertyBag& InValueB, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<float, EPropertyBagResult> ValueAResult = InValueA.GetValueFloat(PropertyHash);
	const TValueOrError<float, EPropertyBagResult> ValueBResult = InValueB.GetValueFloat(PropertyHash);
	if (!ValueAResult.HasValue() || !ValueBResult.HasValue())
	{
		return false;
	}

	const float NewValue = ValueAResult.GetValue() + ValueBResult.GetValue();
	EPropertyBagResult Result = OutValue.SetValueFloat(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreFloatHandler::AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<float, EPropertyBagResult> ValueResult = InValue.GetValueFloat(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	float Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	float NewValue = Value + ValueResult.GetValue();
	const EPropertyBagResult Result = OutValue.SetValueFloat(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreFloatHandler::SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValueA, const FInstancedPropertyBag& InValueB, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<float, EPropertyBagResult> ValueAResult = InValueA.GetValueFloat(PropertyHash);
	const TValueOrError<float, EPropertyBagResult> ValueBResult = InValueB.GetValueFloat(PropertyHash);
	if (!ValueAResult.HasValue() || !ValueBResult.HasValue())
	{
		return false;
	}

	const float NewValue = ValueAResult.GetValue() - ValueBResult.GetValue();
	EPropertyBagResult Result = OutValue.SetValueFloat(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreFloatHandler::SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<float, EPropertyBagResult> ValueResult = InValue.GetValueFloat(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	float Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	float NewValue = Value - ValueResult.GetValue();
	InPropertyData.SetPropertyValuePtr(&NewValue);

	return true;
}

bool UPropertyAnimatorCoreFloatHandler::SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<float, EPropertyBagResult> ValueResult = InValue.GetValueFloat(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	float Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	const float NewValue = Value - ValueResult.GetValue();
	const EPropertyBagResult Result = OutValue.SetValueFloat(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreFloatHandler::GetDefaultValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	OutValue.AddProperty(PropertyHash, EPropertyBagPropertyType::Float);
	OutValue.SetValueFloat(PropertyHash, 0.f);
	return true;
}
