// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Properties/Handlers/PropertyAnimatorCoreHandlerBase.h"
#include "PropertyAnimatorCoreFloatHandler.generated.h"

/** Handles any float property values, get and set handler */
UCLASS(Transient)
class UPropertyAnimatorCoreFloatHandler : public UPropertyAnimatorCoreHandlerBase
{
	GENERATED_BODY()

public:
	//~ Begin UPropertyAnimatorCoreHandlerBase
	virtual bool IsPropertySupported(const FPropertyAnimatorCoreData& InPropertyData) const override;
	virtual bool GetValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue) override;
	virtual bool SetValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue) override;
	virtual bool IsAdditiveSupported() const override;
	virtual bool AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue) override;
	virtual bool AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValueA, const FInstancedPropertyBag&InValueB, FInstancedPropertyBag& OutValue) override;
	virtual bool AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue, FInstancedPropertyBag& OutValue) override;
	virtual bool SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValueA, const FInstancedPropertyBag& InValueB, FInstancedPropertyBag& OutValue) override;
	virtual bool SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue) override;
	virtual bool SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue, FInstancedPropertyBag& OutValue) override;
	virtual bool GetDefaultValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue) override;
	//~ End UPropertyAnimatorCoreHandlerBase
};