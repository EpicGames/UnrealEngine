// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/Handlers/PropertyAnimatorCoreIntHandler.h"

bool UPropertyAnimatorCoreIntHandler::IsPropertySupported(const FPropertyAnimatorCoreData& InPropertyData) const
{
	if (InPropertyData.IsA<FIntProperty>())
	{
		return true;
	}

	return Super::IsPropertySupported(InPropertyData);
}

bool UPropertyAnimatorCoreIntHandler::GetValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	OutValue.AddProperty(PropertyHash, EPropertyBagPropertyType::Int32);

	int32 Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	OutValue.SetValueInt32(PropertyHash, Value);

	return true;
}

bool UPropertyAnimatorCoreIntHandler::SetValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	TValueOrError<int32, EPropertyBagResult> ValueResult = InValue.GetValueInt32(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	int32& NewValue = ValueResult.GetValue();
	InPropertyData.SetPropertyValuePtr(&NewValue);

	return true;
}

bool UPropertyAnimatorCoreIntHandler::IsAdditiveSupported() const
{
	return true;
}

bool UPropertyAnimatorCoreIntHandler::AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<int32, EPropertyBagResult> ValueResult = InValue.GetValueInt32(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	int32 Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	int32 NewValue = Value + ValueResult.GetValue();
	InPropertyData.SetPropertyValuePtr(&NewValue);

	return true;
}

bool UPropertyAnimatorCoreIntHandler::AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValueA, const FInstancedPropertyBag&InValueB, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<int32, EPropertyBagResult> ValueAResult = InValueA.GetValueInt32(PropertyHash);
	const TValueOrError<int32, EPropertyBagResult> ValueBResult = InValueB.GetValueInt32(PropertyHash);
	if (!ValueAResult.HasValue() || !ValueBResult.HasValue())
	{
		return false;
	}

	const int32 NewValue = ValueAResult.GetValue() + ValueBResult.GetValue();
	EPropertyBagResult Result = OutValue.SetValueInt32(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreIntHandler::AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<int32, EPropertyBagResult> ValueResult = InValue.GetValueInt32(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	int32 Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	int32 NewValue = Value + ValueResult.GetValue();
	const EPropertyBagResult Result = OutValue.SetValueInt32(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreIntHandler::SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValueA, const FInstancedPropertyBag& InValueB, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<int32, EPropertyBagResult> ValueAResult = InValueA.GetValueInt32(PropertyHash);
	const TValueOrError<int32, EPropertyBagResult> ValueBResult = InValueB.GetValueInt32(PropertyHash);
	if (!ValueAResult.HasValue() || !ValueBResult.HasValue())
	{
		return false;
	}

	const int32 NewValue = ValueAResult.GetValue() - ValueBResult.GetValue();
	EPropertyBagResult Result = OutValue.SetValueInt32(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreIntHandler::SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<int32, EPropertyBagResult> ValueResult = InValue.GetValueInt32(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	int32 Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	int32 NewValue = Value - ValueResult.GetValue();
	InPropertyData.SetPropertyValuePtr(&NewValue);

	return true;
}

bool UPropertyAnimatorCoreIntHandler::SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<int32, EPropertyBagResult> ValueResult = InValue.GetValueInt32(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	int32 Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	const int32 NewValue = Value - ValueResult.GetValue();
	const EPropertyBagResult Result = OutValue.SetValueInt32(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreIntHandler::GetDefaultValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	OutValue.AddProperty(PropertyHash, EPropertyBagPropertyType::Int32);
	OutValue.SetValueInt32(PropertyHash, 0);
	return true;
}
