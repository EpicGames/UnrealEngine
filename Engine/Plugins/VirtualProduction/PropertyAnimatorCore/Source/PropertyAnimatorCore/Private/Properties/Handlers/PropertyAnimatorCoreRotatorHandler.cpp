// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/Handlers/PropertyAnimatorCoreRotatorHandler.h"

bool UPropertyAnimatorCoreRotatorHandler::IsPropertySupported(const FPropertyAnimatorCoreData& InPropertyData) const
{
	if (InPropertyData.IsA<FStructProperty>() && InPropertyData.GetLeafPropertyTypeName() == NAME_Rotator)
	{
		return true;
	}

	return Super::IsPropertySupported(InPropertyData);
}

bool UPropertyAnimatorCoreRotatorHandler::GetValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	OutValue.AddProperty(PropertyHash, EPropertyBagPropertyType::Struct, TBaseStructure<FRotator>::Get());

	FRotator Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	OutValue.SetValueStruct(PropertyHash, Value);

	return true;
}

bool UPropertyAnimatorCoreRotatorHandler::SetValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	TValueOrError<FRotator*, EPropertyBagResult> ValueResult = InValue.GetValueStruct<FRotator>(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	FRotator* NewValue = ValueResult.GetValue();
	InPropertyData.SetPropertyValuePtr(NewValue);

	return true;
}

bool UPropertyAnimatorCoreRotatorHandler::IsAdditiveSupported() const
{
	return true;
}

bool UPropertyAnimatorCoreRotatorHandler::AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<FRotator*, EPropertyBagResult> ValueResult = InValue.GetValueStruct<FRotator>(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	FRotator Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	FRotator NewValue = FRotator(Value.Quaternion() * ValueResult.GetValue()->Quaternion());
	InPropertyData.SetPropertyValuePtr(&NewValue);

	return true;
}

bool UPropertyAnimatorCoreRotatorHandler::AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValueA, const FInstancedPropertyBag&InValueB, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<FRotator*, EPropertyBagResult> ValueAResult = InValueA.GetValueStruct<FRotator>(PropertyHash);
	const TValueOrError<FRotator*, EPropertyBagResult> ValueBResult = InValueB.GetValueStruct<FRotator>(PropertyHash);
	if (!ValueAResult.HasValue() || !ValueBResult.HasValue())
	{
		return false;
	}

	const FRotator NewValue = FRotator(ValueAResult.GetValue()->Quaternion() * ValueBResult.GetValue()->Quaternion());
	EPropertyBagResult Result = OutValue.SetValueStruct(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreRotatorHandler::AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<FRotator*, EPropertyBagResult> ValueResult = InValue.GetValueStruct<FRotator>(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	FRotator Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	const FRotator NewValue = FRotator(Value.Quaternion() * ValueResult.GetValue()->Quaternion());
	EPropertyBagResult Result = OutValue.SetValueStruct(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreRotatorHandler::SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValueA, const FInstancedPropertyBag& InValueB, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<FRotator*, EPropertyBagResult> ValueAResult = InValueA.GetValueStruct<FRotator>(PropertyHash);
	const TValueOrError<FRotator*, EPropertyBagResult> ValueBResult = InValueB.GetValueStruct<FRotator>(PropertyHash);
	if (!ValueAResult.HasValue() || !ValueBResult.HasValue())
	{
		return false;
	}

	const FRotator NewValue = FRotator(ValueAResult.GetValue()->Quaternion() * ValueBResult.GetValue()->Quaternion().Inverse());
	EPropertyBagResult Result = OutValue.SetValueStruct(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreRotatorHandler::SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<FRotator*, EPropertyBagResult> ValueResult = InValue.GetValueStruct<FRotator>(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	FRotator Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	FRotator NewValue = FRotator(Value.Quaternion() * ValueResult.GetValue()->Quaternion().Inverse());
	InPropertyData.SetPropertyValuePtr(&NewValue);

	return true;
}

bool UPropertyAnimatorCoreRotatorHandler::SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<FRotator*, EPropertyBagResult> ValueResult = InValue.GetValueStruct<FRotator>(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	FRotator Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	const FRotator NewValue = FRotator(Value.Quaternion() * ValueResult.GetValue()->Quaternion().Inverse());
	const EPropertyBagResult Result = OutValue.SetValueStruct(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreRotatorHandler::GetDefaultValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	OutValue.AddProperty(PropertyHash, EPropertyBagPropertyType::Struct, TBaseStructure<FRotator>::Get());
	OutValue.SetValueStruct(PropertyHash, FRotator::ZeroRotator);
	return true;
}
