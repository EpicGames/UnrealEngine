// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/Handlers/PropertyAnimatorCoreStringHandler.h"

bool UPropertyAnimatorCoreStringHandler::IsPropertySupported(const FPropertyAnimatorCoreData& InPropertyData) const
{
	if (InPropertyData.IsA<FStrProperty>())
	{
		return true;
	}

	return Super::IsPropertySupported(InPropertyData);
}

bool UPropertyAnimatorCoreStringHandler::GetValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	OutValue.AddProperty(PropertyHash, EPropertyBagPropertyType::String);

	FString Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	OutValue.SetValueString(PropertyHash, Value);

	return true;
}

bool UPropertyAnimatorCoreStringHandler::SetValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	TValueOrError<FString, EPropertyBagResult> ValueResult = InValue.GetValueString(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	FString& NewValue = ValueResult.GetValue();
	InPropertyData.SetPropertyValuePtr(&NewValue);

	return true;
}

bool UPropertyAnimatorCoreStringHandler::GetDefaultValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	OutValue.AddProperty(PropertyHash, EPropertyBagPropertyType::String);
	OutValue.SetValueString(PropertyHash, FString());
	return true;
}
