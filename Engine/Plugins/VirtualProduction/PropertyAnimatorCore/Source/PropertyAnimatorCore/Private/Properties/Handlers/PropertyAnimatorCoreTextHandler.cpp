// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/Handlers/PropertyAnimatorCoreTextHandler.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/TextProperty.h"

bool UPropertyAnimatorCoreTextHandler::IsPropertySupported(const FPropertyAnimatorCoreData& InPropertyData) const
{
	if (InPropertyData.IsA<FTextProperty>())
	{
		return true;
	}

	return Super::IsPropertySupported(InPropertyData);
}

bool UPropertyAnimatorCoreTextHandler::GetValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	OutValue.AddProperty(PropertyHash, EPropertyBagPropertyType::Text);

	FText Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	OutValue.SetValueText(PropertyHash, Value);

	return true;
}

bool UPropertyAnimatorCoreTextHandler::SetValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	TValueOrError<FText, EPropertyBagResult> ValueResult = InValue.GetValueText(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	FText& NewValue = ValueResult.GetValue();
	InPropertyData.SetPropertyValuePtr(&NewValue);

	return true;
}

bool UPropertyAnimatorCoreTextHandler::GetDefaultValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	OutValue.AddProperty(PropertyHash, EPropertyBagPropertyType::Text);
	OutValue.SetValueText(PropertyHash, FText::GetEmpty());
	return true;
}
