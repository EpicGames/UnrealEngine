// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/Handlers/PropertyAnimatorCoreVectorHandler.h"

bool UPropertyAnimatorCoreVectorHandler::IsPropertySupported(const FPropertyAnimatorCoreData& InPropertyData) const
{
	if (InPropertyData.IsA<FStructProperty>() && InPropertyData.GetLeafPropertyTypeName() == NAME_Vector)
	{
		return true;
	}

	return Super::IsPropertySupported(InPropertyData);
}

bool UPropertyAnimatorCoreVectorHandler::GetValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	OutValue.AddProperty(PropertyHash, EPropertyBagPropertyType::Struct, TBaseStructure<FVector>::Get());

	FVector Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	OutValue.SetValueStruct(PropertyHash, Value);

	return true;
}

bool UPropertyAnimatorCoreVectorHandler::SetValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	TValueOrError<FVector*, EPropertyBagResult> ValueResult = InValue.GetValueStruct<FVector>(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	FVector* NewValue = ValueResult.GetValue();
	InPropertyData.SetPropertyValuePtr(NewValue);

	return true;
}

bool UPropertyAnimatorCoreVectorHandler::IsAdditiveSupported() const
{
	return true;
}

bool UPropertyAnimatorCoreVectorHandler::AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<FVector*, EPropertyBagResult> ValueResult = InValue.GetValueStruct<FVector>(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	FVector Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	FVector NewValue = Value + *ValueResult.GetValue();
	InPropertyData.SetPropertyValuePtr(&NewValue);

	return true;
}

bool UPropertyAnimatorCoreVectorHandler::AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValueA, const FInstancedPropertyBag& InValueB, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<FVector*, EPropertyBagResult> ValueAResult = InValueA.GetValueStruct<FVector>(PropertyHash);
	const TValueOrError<FVector*, EPropertyBagResult> ValueBResult = InValueB.GetValueStruct<FVector>(PropertyHash);
	if (!ValueAResult.HasValue() || !ValueBResult.HasValue())
	{
		return false;
	}

	const FVector NewValue = *ValueAResult.GetValue() + *ValueBResult.GetValue();
	const EPropertyBagResult Result = OutValue.SetValueStruct(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreVectorHandler::AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<FVector*, EPropertyBagResult> ValueResult = InValue.GetValueStruct<FVector>(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	FVector Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	const FVector NewValue = Value + *ValueResult.GetValue();
	EPropertyBagResult Result = OutValue.SetValueStruct(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreVectorHandler::SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValueA, const FInstancedPropertyBag& InValueB, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<FVector*, EPropertyBagResult> ValueAResult = InValueA.GetValueStruct<FVector>(PropertyHash);
	const TValueOrError<FVector*, EPropertyBagResult> ValueBResult = InValueB.GetValueStruct<FVector>(PropertyHash);
	if (!ValueAResult.HasValue() || !ValueBResult.HasValue())
	{
		return false;
	}

	const FVector NewValue = *ValueAResult.GetValue() - *ValueBResult.GetValue();
	EPropertyBagResult Result = OutValue.SetValueStruct(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreVectorHandler::SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<FVector*, EPropertyBagResult> ValueResult = InValue.GetValueStruct<FVector>(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	FVector Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	FVector NewValue = Value - *ValueResult.GetValue();
	InPropertyData.SetPropertyValuePtr(&NewValue);

	return true;
}

bool UPropertyAnimatorCoreVectorHandler::SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	const TValueOrError<FVector*, EPropertyBagResult> ValueResult = InValue.GetValueStruct<FVector>(PropertyHash);
	if (!ValueResult.HasValue())
	{
		return false;
	}

	FVector Value;
	InPropertyData.GetPropertyValuePtr(&Value);

	const FVector NewValue = Value - *ValueResult.GetValue();
	const EPropertyBagResult Result = OutValue.SetValueStruct(PropertyHash, NewValue);

	return Result == EPropertyBagResult::Success;
}

bool UPropertyAnimatorCoreVectorHandler::GetDefaultValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue)
{
	const FName PropertyHash(InPropertyData.GetLocatorPathHash());
	OutValue.AddProperty(PropertyHash, EPropertyBagPropertyType::Struct, TBaseStructure<FVector>::Get());
	OutValue.SetValueStruct(PropertyHash, FVector::ZeroVector);
	return true;
}
