// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/PropertyAnimatorCoreContext.h"

#include "Containers/Ticker.h"
#include "Logs/PropertyAnimatorCoreLog.h"
#include "Properties/PropertyAnimatorCoreResolver.h"
#include "Properties/Converters/PropertyAnimatorCoreConverterBase.h"
#include "Properties/Handlers/PropertyAnimatorCoreHandlerBase.h"
#include "StructUtils/InstancedStruct.h"
#include "Subsystems/PropertyAnimatorCoreSubsystem.h"

TArray<FPropertyAnimatorCoreData> UPropertyAnimatorCoreContext::ResolveProperty(bool bInForEvaluation) const
{
	TArray<FPropertyAnimatorCoreData> ResolvedProperties;

	if (UPropertyAnimatorCoreResolver* PropertyResolver = GetResolver())
	{
		PropertyResolver->ResolveTemplateProperties(AnimatedProperty, ResolvedProperties, bInForEvaluation);
	}
	else
	{
		ResolvedProperties.Add(AnimatedProperty);
	}

	return ResolvedProperties;
}

FName UPropertyAnimatorCoreContext::GetAnimatedPropertyName()
{
	return GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreContext, AnimatedProperty);
}

UPropertyAnimatorCoreBase* UPropertyAnimatorCoreContext::GetAnimator() const
{
	return GetTypedOuter<UPropertyAnimatorCoreBase>();
}

UPropertyAnimatorCoreHandlerBase* UPropertyAnimatorCoreContext::GetHandler() const
{
	if (!HandlerWeak.IsValid())
	{
		if (const UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get())
		{
			UPropertyAnimatorCoreContext* MutableThis = const_cast<UPropertyAnimatorCoreContext*>(this);
			MutableThis->HandlerWeak = AnimatorSubsystem->GetHandler(AnimatedProperty);
		}
	}

	return HandlerWeak.Get();
}

UPropertyAnimatorCoreResolver* UPropertyAnimatorCoreContext::GetResolver() const
{
	if (Resolver)
	{
		return Resolver;
	}

	return AnimatedProperty.GetPropertyResolver();
}

bool UPropertyAnimatorCoreContext::IsResolvable() const
{
	return AnimatedProperty.IsResolvable();
}

bool UPropertyAnimatorCoreContext::IsConverted() const
{
	return !!ConverterClass.Get();
}

void UPropertyAnimatorCoreContext::SetAnimated(bool bInAnimated)
{
	if (bAnimated == bInAnimated)
	{
		return;
	}

	bAnimated = bInAnimated;
	OnAnimatedChanged();
}

void UPropertyAnimatorCoreContext::SetMagnitude(float InMagnitude)
{
	Magnitude = FMath::Clamp(InMagnitude, 0.f, 1.f);
}

void UPropertyAnimatorCoreContext::SetTimeOffset(double InOffset)
{
	TimeOffset = InOffset;
}

void UPropertyAnimatorCoreContext::SetMode(EPropertyAnimatorCoreMode InMode)
{
	if (InMode == Mode)
	{
		return;
	}

	Restore();
	Mode = InMode;
	OnModeChanged();
}

void UPropertyAnimatorCoreContext::SetConverterClass(const TSubclassOf<UPropertyAnimatorCoreConverterBase>& InConverterClass)
{
	ConverterClass = InConverterClass;

	if (const UPropertyAnimatorCoreConverterBase* Converter = InConverterClass.GetDefaultObject())
	{
		if (const UScriptStruct* RuleStruct = Converter->GetConversionRuleStruct())
		{
			ConverterRule.InitializeAsScriptStruct(RuleStruct);
			CheckEditConverterRule();
		}
	}
}

void UPropertyAnimatorCoreContext::PostLoad()
{
	Super::PostLoad();

	CheckEditConditions();

	if (UPropertyAnimatorCoreResolver* PropertyResolver = AnimatedProperty.GetPropertyResolver())
	{
		TSet<FPropertyAnimatorCoreData> ResolvedProperties;
		if (PropertyResolver->FixUpProperty(AnimatedProperty))
		{
			UE_LOGF(LogPropertyAnimatorCore, Log, "Fixed up property %ls using %ls resolver", *AnimatedProperty.GetPropertyDisplayName(), *PropertyResolver->GetResolverName().ToString())
		}
	}

	AnimatedProperty.GeneratePropertyPath();

	for (const FPropertyAnimatorCoreData& Property : ResolveProperty(/** Evaluation */false))
	{
		const FName OldPropertyPath(Property.GetPathHash());
		const FName NewPropertyPath(Property.GetLocatorPathHash());

		if (DeltaPropertyValues.RenameProperty(OldPropertyPath, NewPropertyPath) == EPropertyBagAlterationResult::Success)
		{
			UE_LOGF(LogPropertyAnimatorCore, Log, "Property %ls in delta property bag migrated successfully %ls to %ls", *Property.GetPropertyDisplayName(), *OldPropertyPath.ToString(), *NewPropertyPath.ToString());
		}

		if (OriginalPropertyValues.RenameProperty(OldPropertyPath, NewPropertyPath) == EPropertyBagAlterationResult::Success)
		{
			UE_LOGF(LogPropertyAnimatorCore, Log, "Property %ls in absolute property bag migrated successfully %ls to %ls", *Property.GetPropertyDisplayName(), *OldPropertyPath.ToString(), *NewPropertyPath.ToString());
		}
	}

	/** Restore property values on next tick */
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateWeakLambda(this, [this](float)
	{
		Restore();

		/** Stop ticker */
		return false;
	}));
}

#if WITH_EDITOR
void UPropertyAnimatorCoreContext::PreEditChange(FProperty* InPropertyAboutToChange)
{
	Super::PreEditChange(InPropertyAboutToChange);

	if (!InPropertyAboutToChange)
	{
		return;
	}

	const FName MemberName = InPropertyAboutToChange->GetFName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreContext, Mode))
	{
		Restore();
	}
}

void UPropertyAnimatorCoreContext::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreContext, Mode))
	{
		OnModeChanged();
	}
	else if (MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreContext, bAnimated))
	{
		OnAnimatedChanged();
	}
}
#endif

bool UPropertyAnimatorCoreContext::ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue)
{
	TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> ObjectArchive = InValue->AsMutableObject();

	if (!ObjectArchive)
	{
		return false;
	}

	bool bAnimatedArchive = bAnimated;
	ObjectArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreContext, bAnimated), bAnimatedArchive);
	SetAnimated(bAnimatedArchive);

	if (bEditMagnitude)
	{
		double MagnitudeArchive = Magnitude;
		ObjectArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreContext, Magnitude), MagnitudeArchive);
		SetMagnitude(MagnitudeArchive);

		double TimeOffsetArchive = TimeOffset;
		ObjectArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreContext, TimeOffset), TimeOffsetArchive);
		SetTimeOffset(TimeOffsetArchive);
	}

	if (bEditMode)
	{
		int64 ModeArchive = static_cast<int64>(Mode);
		ObjectArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreContext, Mode), ModeArchive);
		SetMode(static_cast<EPropertyAnimatorCoreMode>(ModeArchive));
	}

	if (Resolver)
	{
		if (ObjectArchive->Has(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreContext, Resolver), EPropertyAnimatorCorePresetArchiveType::Object))
		{
			TSharedPtr<FPropertyAnimatorCorePresetArchive> ResolverArchive;
			ObjectArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreContext, Resolver), ResolverArchive);
			Resolver->ImportPreset(InPreset, ResolverArchive.ToSharedRef());
		}
	}

	return true;
}

bool UPropertyAnimatorCoreContext::ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const
{
	TSharedRef<FPropertyAnimatorCorePresetObjectArchive> ContextArchive = InPreset->GetArchiveImplementation()->CreateObject();
	OutValue = ContextArchive;

	ContextArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreContext, bAnimated), bAnimated);

	if (bEditMagnitude)
	{
		ContextArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreContext, Magnitude), Magnitude);
		ContextArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreContext, TimeOffset), TimeOffset);
	}

	if (bEditMode)
	{
		ContextArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreContext, Mode), static_cast<uint64>(Mode));
	}

	ContextArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreContext, AnimatedProperty), AnimatedProperty.GetLocatorPath());

	if (Resolver)
	{
		TSharedPtr<FPropertyAnimatorCorePresetArchive> ResolverArchive;
		if (Resolver->ExportPreset(InPreset, ResolverArchive) && ResolverArchive.IsValid())
		{
			ContextArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreContext, Resolver), ResolverArchive.ToSharedRef());
		}
	}

	return true;
}

void UPropertyAnimatorCoreContext::OnAnimatedPropertyLinked()
{
	if (const UPropertyAnimatorCoreResolver* PropertyResolver = AnimatedProperty.GetPropertyResolver())
	{
		if (!PropertyResolver->GetClass()->HasAnyClassFlags(CLASS_Abstract | CLASS_Transient))
		{
			bEditResolver = true;
			Resolver = NewObject<UPropertyAnimatorCoreResolver>(this, PropertyResolver->GetClass());
		}
	}
}

void UPropertyAnimatorCoreContext::OnModeChanged()
{
	if (const UPropertyAnimatorCoreHandlerBase* Handler = GetHandler())
	{
		if (Mode == EPropertyAnimatorCoreMode::Additive && !Handler->IsAdditiveSupported())
		{
			Mode = EPropertyAnimatorCoreMode::Absolute;
		}

		Save();
	}
}

bool UPropertyAnimatorCoreContext::ResolvePropertyOwner(AActor* InNewOwner)
{
	AActor* NewOwningActor = InNewOwner ? InNewOwner : GetTypedOuter<AActor>();
	const UObject* CurrentOwningActor = AnimatedProperty.GetOwningActor();

	if (CurrentOwningActor == NewOwningActor)
	{
		return true;
	}

	const bool bFound = IsValid(NewOwningActor);

	// Try to resolve property owner on new owning actor
	UObject* NewOwner = FPropertyAnimatorCoreData(NewOwningActor, AnimatedProperty.GetLocatorPath()).GetOwner();

	const FProperty* MemberProperty = AnimatedProperty.GetMemberProperty();
	UClass* PropertyOwningClass = MemberProperty->GetOwnerClass();

	if (bFound
		&& IsValid(NewOwner)
		&& NewOwner->GetClass()->IsChildOf(PropertyOwningClass)
		&& FindFProperty<FProperty>(NewOwner->GetClass(), AnimatedProperty.GetMemberPropertyName()))
	{
		SetAnimatedPropertyOwner(NewOwner);
		return true;
	}

	UE_LOGF(LogPropertyAnimatorCore, Warning, "Could not resolve property %ls on actor %ls", *AnimatedProperty.GetLocatorPath(), NewOwningActor ? *NewOwningActor->GetActorNameOrLabel() : TEXT("Invalid"))

	return false;
}

void UPropertyAnimatorCoreContext::ConstructInternal(const FPropertyAnimatorCoreData& InProperty)
{
	AnimatedProperty = InProperty;
	CheckEditConditions();
	SetMode(EPropertyAnimatorCoreMode::Additive);
	OnAnimatedPropertyLinked();
}

void UPropertyAnimatorCoreContext::SetAnimatedPropertyOwner(UObject* InNewOwner)
{
	if (!IsValid(InNewOwner))
	{
		return;
	}

	if (!FindFProperty<FProperty>(InNewOwner->GetClass(), AnimatedProperty.GetMemberPropertyName()))
	{
		return;
	}

	constexpr bool bEvenIfPendingKill = true;
	UObject* PreviousOwner = AnimatedProperty.GetOwnerWeak().Get(bEvenIfPendingKill);
	AnimatedProperty = FPropertyAnimatorCoreData(InNewOwner, AnimatedProperty.GetChainProperties(), AnimatedProperty.GetPropertyResolverClass());

	OnAnimatedPropertyOwnerUpdated(PreviousOwner, InNewOwner);
}

void UPropertyAnimatorCoreContext::CheckEditMode()
{
	if (const UPropertyAnimatorCoreHandlerBase* Handler = GetHandler())
	{
		bEditMode = Handler->IsAdditiveSupported();
	}
}

void UPropertyAnimatorCoreContext::CheckEditConverterRule()
{
	bEditConverterRule = ConverterRule.IsValid();
}

void UPropertyAnimatorCoreContext::CheckEditResolver()
{
	bEditResolver = AnimatedProperty.IsResolvable();
}

void* UPropertyAnimatorCoreContext::GetConverterRulePtr(const UScriptStruct* InStruct)
{
	if (ConverterRule.IsValid() && ConverterRule.GetScriptStruct()->IsChildOf(InStruct))
	{
		return ConverterRule.GetMutableMemory();
	}

	return nullptr;
}

void UPropertyAnimatorCoreContext::CheckEditConditions()
{
	CheckEditMagnitude();
	CheckEditTimeOffset();
	CheckEditMode();
	CheckEditConverterRule();
	CheckEditResolver();
}

void UPropertyAnimatorCoreContext::CheckEditMagnitude()
{
	bEditMagnitude = AnimatedProperty.IsA<FNumericProperty>()
		|| AnimatedProperty.HasA<FNumericProperty>()
		|| AnimatedProperty.IsA<FBoolProperty>();
}

void UPropertyAnimatorCoreContext::CheckEditTimeOffset()
{
	bEditTimeOffset = bEditMagnitude;	
}

void UPropertyAnimatorCoreContext::Restore()
{
	if (OriginalPropertyValues.GetNumPropertiesInBag() == 0
		&& DeltaPropertyValues.GetNumPropertiesInBag() == 0)
	{
		return;
	}

	UPropertyAnimatorCoreHandlerBase* Handler = GetHandler();

	if (!Handler)
	{
		return;
	}

	if (Mode == EPropertyAnimatorCoreMode::Absolute)
	{
		for (const FPropertyAnimatorCoreData& ResolvedProperty : ResolveProperty(false))
		{
			// Set original value
			Handler->SetValue(ResolvedProperty, OriginalPropertyValues);
		}
	}
	else
	{
		for (const FPropertyAnimatorCoreData& ResolvedProperty : ResolveProperty(false))
		{
			// Subtract delta value
			Handler->SubtractValue(ResolvedProperty, DeltaPropertyValues);
		}
	}

	OriginalPropertyValues.Reset();
	DeltaPropertyValues.Reset();
}

void UPropertyAnimatorCoreContext::Save()
{
	UPropertyAnimatorCoreHandlerBase* Handler = GetHandler();

	if (!Handler)
	{
		return;
	}

	for (const FPropertyAnimatorCoreData& PropertyData : ResolveProperty(false))
	{
		const FName Name(PropertyData.GetLocatorPathHash());
		if (!OriginalPropertyValues.FindPropertyDescByName(Name))
		{
			const FProperty* Property = PropertyData.GetLeafProperty();
			OriginalPropertyValues.AddProperty(Name, Property);

			// Save original value
			Handler->GetValue(PropertyData, OriginalPropertyValues);
		}

		if (!DeltaPropertyValues.FindPropertyDescByName(Name))
		{
			const FProperty* Property = PropertyData.GetLeafProperty();
			DeltaPropertyValues.AddProperty(Name, Property);

			// Save default value (zero)
			Handler->GetDefaultValue(PropertyData, DeltaPropertyValues);
		}
	}
}

void UPropertyAnimatorCoreContext::OnAnimatedChanged()
{
	if (!bAnimated)
	{
		Restore();
	}
}

void UPropertyAnimatorCoreContext::CommitEvaluationResult(const FPropertyAnimatorCoreData& InResolvedProperty, FInstancedPropertyBag& InEvaluatedValues)
{
	if (!IsAnimated())
	{
		return;
	}

	UPropertyAnimatorCoreHandlerBase* Handler = GetHandler();

	if (!Handler)
	{
		return;
	}

	const FName PropertyName(InResolvedProperty.GetLocatorPathHash());

	const FPropertyBagPropertyDesc* FromProperty = InEvaluatedValues.FindPropertyDescByName(PropertyName);
	const FPropertyBagPropertyDesc* ToProperty = DeltaPropertyValues.FindPropertyDescByName(PropertyName);

	checkf(!!FromProperty && !!ToProperty, TEXT("Property bags are missing the evaluated property"));

	if (UPropertyAnimatorCoreConverterBase* Converter = ConverterClass.GetDefaultObject())
	{
		if (!Converter->Convert(*FromProperty, InEvaluatedValues, *ToProperty, InEvaluatedValues, ConverterRule))
		{
			return;
		}
	}

	if (Mode == EPropertyAnimatorCoreMode::Additive)
	{
		// Original = Current - Previous
		Handler->SubtractValue(InResolvedProperty, DeltaPropertyValues, DeltaPropertyValues);

		// NewValue = Original + Evaluated
		Handler->AddValue(InResolvedProperty, DeltaPropertyValues, InEvaluatedValues, InEvaluatedValues);

		// Delta = NewValue - Original
		Handler->SubtractValue(InResolvedProperty, InEvaluatedValues, DeltaPropertyValues, DeltaPropertyValues);
	}

	Handler->SetValue(InResolvedProperty, InEvaluatedValues);

	// Done with the value
	InEvaluatedValues.RemovePropertyByName(PropertyName);
}
