// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/PropertyAnimatorCoreData.h"

#include "Properties/PropertyAnimatorCoreResolver.h"
#include "Subsystems/PropertyAnimatorCoreSubsystem.h"

FPropertyAnimatorCoreData::FPropertyAnimatorCoreData(UObject* InObject, FProperty* InMemberProperty, FProperty* InProperty, TSubclassOf<UPropertyAnimatorCoreResolver> InResolverClass)
	: OwnerWeak(InObject)
	, PropertyResolverClass(InResolverClass)
{
	if (InMemberProperty && InProperty && InMemberProperty == InProperty)
	{
		InProperty = nullptr;
	}

	for (FProperty* CurrentProperty = InProperty; CurrentProperty && CurrentProperty != InMemberProperty;)
	{
		ChainProperties.Add(CurrentProperty);
		CurrentProperty = CurrentProperty->GetOwner<FProperty>();
	}

	if (InMemberProperty)
	{
		ChainProperties.Push(InMemberProperty);
	}

	Algo::Reverse(ChainProperties);

	GeneratePropertyPath();
	FindSetterFunctions();
}

FPropertyAnimatorCoreData::FPropertyAnimatorCoreData(UObject* InObject, const TArray<FProperty*>& InChainProperties, TSubclassOf<UPropertyAnimatorCoreResolver> InResolverClass)
	: OwnerWeak(InObject)
	, PropertyResolverClass(InResolverClass)
{
	for (FProperty* ChainProperty : InChainProperties)
	{
		if (ChainProperty)
		{
			ChainProperties.Add(ChainProperty);
		}
	}

	GeneratePropertyPath();
	FindSetterFunctions();
}

FPropertyAnimatorCoreData::FPropertyAnimatorCoreData(UObject* InObject, const TArray<FProperty*>& InChainProperties, FProperty* InProperty, TSubclassOf<UPropertyAnimatorCoreResolver> InResolverClass)
	: OwnerWeak(InObject)
	, PropertyResolverClass(InResolverClass)
{
	for (FProperty* ChainProperty : InChainProperties)
	{
		if (ChainProperty)
		{
			ChainProperties.Add(ChainProperty);
		}
	}

	if (InProperty)
	{
		ChainProperties.AddUnique(InProperty);
	}

	GeneratePropertyPath();
	FindSetterFunctions();
}

FPropertyAnimatorCoreData::FPropertyAnimatorCoreData(AActor* InActor, const FString& InPropertyLocatorPath)
{
	UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();

	if (!InActor || InPropertyLocatorPath.IsEmpty() || !Subsystem)
	{
		return;
	}

	UPropertyAnimatorCoreResolver* Resolver = nullptr;

	TArray<FString> Elements;
	InPropertyLocatorPath.ParseIntoArray(Elements, TEXT(";"));

	if (Elements.Num() < 2)
	{
		return;
	}

	int32 ElementIndex = 0;

	// Get Resolver
	if (Elements.Num() == 3)
	{
		FString ResolverClass;
		FString ResolverName;
		Elements[ElementIndex++].Split(TEXT(":"), &ResolverClass, &ResolverName);

		Resolver = Subsystem->FindResolverByName(FName(ResolverName));
		Resolver = Resolver ? Resolver : Subsystem->FindResolverByClass(LoadObject<UClass>(nullptr, *ResolverClass));

		if (Resolver)
		{
			PropertyResolverClass = Resolver->GetClass();
		}
	}

	// Locate Owner
	TArray<FString> Outers;
	Elements[ElementIndex++].ParseIntoArray(Outers, TEXT(","));

	// Locate Property
	TArray<FString> Properties;
	Elements[ElementIndex].ParseIntoArray(Properties, TEXT(","));

	for (const FString& PropertyPath : Properties)
	{
		FProperty* Property = FindFProperty<FProperty>(*PropertyPath);

		if (!Property)
		{
			break;
		}

		ChainProperties.Add(Property);
	}

	// Search for an object containing member property
	if (const FProperty* MemberProperty = GetMemberProperty())
	{
		UClass* MemberOwningClass = MemberProperty->GetOwnerClass();

		UObject* FoundObject = InActor;
		for (int32 Index = 0; Index < Outers.Num(); Index++)
		{
			FString OuterClass;
			FString OuterName;
			Outers[Index].Split(TEXT(":"), &OuterClass, &OuterName);

			TArray<UObject*> OwnedObjects;
			GetObjectsWithOuter(FoundObject, OwnedObjects, EGetObjectsFlags::None);

			bool bFound = false;

			// Search by name
			for (UObject* OwnedObject : OwnedObjects)
			{
				if (OwnedObject->GetName() == OuterName)
				{
					FoundObject = OwnedObject;
					bFound = true;
					break;
				}
			}

			// Search by class
			if (!bFound)
			{
				for (UObject* OwnedObject : OwnedObjects)
				{
					if (OwnedObject->GetClass()->GetClassPathName().ToString() == OuterClass)
					{
						FoundObject = OwnedObject;
						bFound = true;
						break;
					}
				}
			}

			// Search by parent class when last outer to allow (USceneComponent::RelativeLocation == UStaticMeshComponent::RelativeLocation)
			if (!bFound && (Outers.Num() - 1) == Index && MemberProperty)
			{
				for (UObject* OwnedObject : OwnedObjects)
				{
					if (OwnedObject->GetClass()->IsChildOf(MemberOwningClass))
					{
						FoundObject = OwnedObject;
						bFound = true;
						break;
					}
				}
			}

			if (!bFound)
			{
				FoundObject = nullptr;
				break;
			}
		}

		OwnerWeak = FoundObject;

		// Use resolver if owner was not found
		if (!FoundObject && Resolver && MemberProperty)
		{
			TSet<FPropertyAnimatorCoreData> ResolvableProperties;
			Resolver->GetTemplateProperties(InActor, ResolvableProperties);

			for (const FPropertyAnimatorCoreData& ResolvableProperty : ResolvableProperties)
			{
				UObject* PropertyOwner = ResolvableProperty.GetOwner();

				if (!PropertyOwner)
				{
					continue;
				}

				if (PropertyOwner->IsA(MemberOwningClass))
				{
					FoundObject = PropertyOwner;
					break;
				}
			}

			OwnerWeak = FoundObject;
		}
	}

	GeneratePropertyPath();
	FindSetterFunctions();
}

bool FPropertyAnimatorCoreData::IsResolvable() const
{
	return PropertyResolverClass.Get() != nullptr;
}

UPropertyAnimatorCoreResolver* FPropertyAnimatorCoreData::GetPropertyResolver() const
{
	if (IsResolvable())
	{
		return PropertyResolverClass.GetDefaultObject();
	}

	return nullptr;
}

TSubclassOf<UPropertyAnimatorCoreResolver> FPropertyAnimatorCoreData::GetPropertyResolverClass() const
{
	return PropertyResolverClass;
}

AActor* FPropertyAnimatorCoreData::GetOwningActor() const
{
	UObject* Owner = GetOwner();

	if (!IsValid(Owner))
	{
		return nullptr;
	}

	if (AActor* const OwningActor = Cast<AActor>(Owner))
	{
		return OwningActor;
	}

	return Owner->GetTypedOuter<AActor>();
}

UActorComponent* FPropertyAnimatorCoreData::GetOwningComponent() const
{
	UObject* Owner = GetOwner();

	if (!IsValid(Owner))
	{
		return nullptr;
	}

	if (UActorComponent* const OwningComponent = Cast<UActorComponent>(Owner))
	{
		return OwningComponent;
	}

	return Owner->GetTypedOuter<UActorComponent>();
}

TArray<UObject*> FPropertyAnimatorCoreData::GetOuters(const UObject* InStopOuter) const
{
	TArray<UObject*> Owners;

	UObject* Outer = OwnerWeak.Get();

	if (!Outer)
	{
		return Owners;
	}

	while (Outer && Outer != InStopOuter)
	{
		Owners.Add(Outer);
		Outer = Outer->GetOuter();
	}

	Algo::Reverse(Owners);

	return Owners;
}

FString FPropertyAnimatorCoreData::GetPropertyDisplayName() const
{
	if (!PropertyDisplayName.IsEmpty())
	{
		return PropertyDisplayName;
	}

	FPropertyAnimatorCoreData* MutableThis = const_cast<FPropertyAnimatorCoreData*>(this);
	MutableThis->GeneratePropertyDisplayName();

	return PropertyDisplayName;
}

FString FPropertyAnimatorCoreData::GetPathHash() const
{
	return PathHash;
}

FString FPropertyAnimatorCoreData::GetLocatorPath() const
{
	if (!LocatorPath.IsEmpty())
	{
		return LocatorPath;
	}

	FPropertyAnimatorCoreData* MutableThis = const_cast<FPropertyAnimatorCoreData*>(this);
	MutableThis->GeneratePropertyLocatorPath();

	return LocatorPath;
}

FName FPropertyAnimatorCoreData::GetLocatorPathHash() const
{
	return FName(LexToString(GetTypeHash(GetLocatorPath())));
}

void FPropertyAnimatorCoreData::GeneratePropertyLocatorPath()
{
	FString NewLocatorPath;

	// Append resolver
	if (const UPropertyAnimatorCoreResolver* Resolver = GetPropertyResolver())
	{
		NewLocatorPath += Resolver->GetClass()->GetClassPathName().ToString() + TEXT(":") + Resolver->GetResolverName().ToString() + TEXT(";");
	}

	// Append outers
	const AActor* OwningActor = GetOwningActor();
	for (const UObject* Outer : GetOuters(OwningActor))
	{
		NewLocatorPath += Outer->GetClass()->GetClassPathName().ToString() + TEXT(":") + Outer->GetName() + TEXT(",");
	}

	NewLocatorPath.RemoveFromEnd(TEXT(","));
	NewLocatorPath += TEXT(";");

	// Append properties
	for (const TFieldPath<FProperty>& Property : ChainProperties)
	{
		NewLocatorPath += Property->GetPathName(OwningActor) + TEXT(",");
	}

	NewLocatorPath.RemoveFromEnd(TEXT(","));

	LocatorPath = NewLocatorPath;
}

void FPropertyAnimatorCoreData::GeneratePropertyDisplayName()
{
	const FString ResolverName = IsResolvable()
		? (GetPropertyResolver()->GetResolverName().ToString() + TEXT("."))
		: TEXT("");

	FString DisplayName;
	FString PropertyTypePath;
	for (const TFieldPath<FProperty>& ChainProperty : ChainProperties)
	{
		FString FriendlyName = ChainProperty->GetName();

		if (ChainProperty->IsA<FBoolProperty>())
		{
			FriendlyName.RemoveFromStart(TEXT("b"), ESearchCase::Type::CaseSensitive);
		}

		DisplayName += DisplayName.IsEmpty()
			? FriendlyName
			: (TEXT(".") + FriendlyName);

		const FString TypeName = GetPropertyTypeName(ChainProperty.Get()).ToString();

		PropertyTypePath += PropertyTypePath.IsEmpty()
			? TypeName
			: (TEXT(".") + TypeName);
	}

	if (!ChainProperties.IsEmpty())
	{
		const FString LeafPropertyName = ChainProperties.Last()->GetName();

		PropertyTypePath += PropertyTypePath.IsEmpty()
			? LeafPropertyName
			: (TEXT(".") + LeafPropertyName);
	}

	// Find alias
	FString AliasName;
	if (const UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get())
	{
		AliasName = AnimatorSubsystem->FindPropertyAlias(PropertyTypePath);
	}

	if (!AliasName.IsEmpty())
	{
		int32 LastPeriodIndex;
		if (DisplayName.FindLastChar(TEXT('.'), LastPeriodIndex))
		{
			DisplayName = DisplayName.Left(LastPeriodIndex + 1);
		}

		DisplayName += AliasName;
	}

	PropertyDisplayName = ResolverName + DisplayName;
}

FName FPropertyAnimatorCoreData::GetMemberPropertyName() const
{
	const FProperty* MemberProperty = GetMemberProperty();
	return MemberProperty ? MemberProperty->GetFName() : NAME_None;
}

FName FPropertyAnimatorCoreData::GetMemberPropertyTypeName() const
{
	return GetPropertyTypeName(GetMemberProperty());
}

FName FPropertyAnimatorCoreData::GetLeafPropertyName() const
{
	const FProperty* LeafProperty = GetLeafProperty();
	return LeafProperty ? LeafProperty->GetFName() : NAME_None;
}

FName FPropertyAnimatorCoreData::GetLeafPropertyTypeName() const
{
	return GetPropertyTypeName(GetLeafProperty());
}

TArray<FProperty*> FPropertyAnimatorCoreData::GetChainProperties() const
{
	TArray<FProperty*> Properties;

	Algo::Transform(
		ChainProperties
		, Properties
		, [](const TFieldPath<FProperty>& InProperty)
		{
			return InProperty.Get();
		}
	);

	return Properties;
}

TArray<FName> FPropertyAnimatorCoreData::GetChainPropertyNames() const
{
	TArray<FName> Properties;

	Algo::Transform(
		ChainProperties
		, Properties
		, [](const TFieldPath<FProperty>& InProperty)
		{
			return InProperty->GetFName();
		}
	);

	return Properties;
}

bool FPropertyAnimatorCoreData::HasSetter() const
{
	FPropertyAnimatorCoreData* This = const_cast<FPropertyAnimatorCoreData*>(this);
	return This->FindSetterFunctions();
}

bool FPropertyAnimatorCoreData::IsParentOf(const FPropertyAnimatorCoreData& InOtherProperty) const
{
	FProperty* LeafProperty = GetLeafProperty();
	const UObject* Owner = GetOwner();

	if (Owner && Owner == InOtherProperty.GetOwner() && LeafProperty)
	{
		const TArray<FProperty*> OtherChainProperties = InOtherProperty.GetChainProperties();

		const int32 LeafIdx = OtherChainProperties.Find(LeafProperty);

		// If leaf property is found and is the penultimate, then we are the parent of other property
		return LeafIdx != INDEX_NONE && LeafIdx == OtherChainProperties.Num() - 2;
	}

	return false;
}

bool FPropertyAnimatorCoreData::IsChildOf(const FPropertyAnimatorCoreData& InOtherProperty) const
{
	return InOtherProperty.IsParentOf(*this);
}

bool FPropertyAnimatorCoreData::IsOwning(const FPropertyAnimatorCoreData& InOtherProperty) const
{
	FProperty* LeafProperty = GetLeafProperty();
	const UObject* Owner = GetOwner();

	if (Owner && Owner == InOtherProperty.GetOwner() && LeafProperty)
	{
		const TArray<FProperty*> OtherChainProperties = InOtherProperty.GetChainProperties();

		const int32 LeafIdx = OtherChainProperties.Find(LeafProperty);

		// If leaf property is found and is not the last property then we contain this other property
		return LeafIdx != INDEX_NONE && LeafIdx != OtherChainProperties.Num() - 1;
	}

	return false;
}

bool FPropertyAnimatorCoreData::IsTransient() const
{
	const UObject* Owner = GetOwner();
	if (Owner && Owner->HasAnyFlags(RF_Transient))
	{
		return true;
	}

	for (const TFieldPath<FProperty>& ChainProperty : ChainProperties)
	{
		const FProperty* Property = ChainProperty.Get();
		if (Property && Property->HasAnyPropertyFlags(CPF_Transient))
		{
			return true;
		}
	}

	return false;
}

TOptional<FPropertyAnimatorCoreData> FPropertyAnimatorCoreData::GetChildOf(const FPropertyAnimatorCoreData& InOtherProperty) const
{
	if (!InOtherProperty.IsOwning(*this))
	{
		return TOptional<FPropertyAnimatorCoreData>();
	}

	FProperty* LeafProperty = InOtherProperty.GetLeafProperty();
	if (!LeafProperty)
	{
		return TOptional<FPropertyAnimatorCoreData>();
	}

	// Find the leaf property of other property inside this property
	const int32 Idx = ChainProperties.Find(LeafProperty);
	if (Idx == INDEX_NONE || Idx == ChainProperties.Num() - 1)
	{
		return TOptional<FPropertyAnimatorCoreData>();
	}

	// Add 1 to get the child of other property
	TArray<FProperty*> ChildChainProperties;
	for (int32 PropIdx = 0; PropIdx <= Idx+1; PropIdx++)
	{
		ChildChainProperties.Add(ChainProperties[PropIdx].Get());
	}

	return FPropertyAnimatorCoreData(GetOwner(), ChildChainProperties, GetPropertyResolverClass());
}

TOptional<FPropertyAnimatorCoreData> FPropertyAnimatorCoreData::GetParent() const
{
	// No parent data available
	if (ChainProperties.Num() == 1)
	{
		return TOptional<FPropertyAnimatorCoreData>();
	}

	TArray<FProperty*> ParentChainProperties;
	for (int32 PropIdx = 0; PropIdx < ChainProperties.Num() - 1; PropIdx++)
	{
		ParentChainProperties.Add(ChainProperties[PropIdx].Get());
	}

	return FPropertyAnimatorCoreData(GetOwner(), ParentChainProperties, GetPropertyResolverClass());
}

TOptional<FPropertyAnimatorCoreData> FPropertyAnimatorCoreData::GetRootParent() const
{
	if (ChainProperties.IsEmpty())
	{
		return TOptional<FPropertyAnimatorCoreData>();
	}

	const TArray<FProperty*> ParentChainProperties { ChainProperties[0].Get() };
	return FPropertyAnimatorCoreData(GetOwner(), ParentChainProperties, GetPropertyResolverClass());
}

TArray<FPropertyAnimatorCoreData> FPropertyAnimatorCoreData::GetChildrenProperties(int32 InDepthSearch) const
{
	TArray<FPropertyAnimatorCoreData> ChildrenProperties;
	FProperty* LeafProperty = GetLeafProperty();

	if (!LeafProperty || InDepthSearch-- <= 0)
	{
		return ChildrenProperties;
	}

	if (const FStructProperty* StructProperty = CastField<FStructProperty>(LeafProperty))
	{
		if (const UScriptStruct* Struct = StructProperty->Struct)
		{
			TArray<FProperty*> Properties;
			Algo::Transform(ChainProperties, Properties, [](const TFieldPath<FProperty>& InProperty)
			{
				return InProperty.Get();
			});

			for (FProperty* ChildProperty : TFieldRange<FProperty>(Struct))
			{
				if (ChildProperty)
				{
					FPropertyAnimatorCoreData ChildPropertyData(GetOwner(), Properties, ChildProperty, GetPropertyResolverClass());

					ChildrenProperties.Add(ChildPropertyData);
					ChildrenProperties.Append(ChildPropertyData.GetChildrenProperties(InDepthSearch));
				}
			}
		}
	}

	// We don't handle object, array, map, set container for now

	return ChildrenProperties;
}

UPropertyAnimatorCoreHandlerBase* FPropertyAnimatorCoreData::GetPropertyHandler() const
{
	// Cache it once
	if (!PropertyHandler)
	{
		if (const UPropertyAnimatorCoreSubsystem* ControlSubsystem = UPropertyAnimatorCoreSubsystem::Get())
		{
			const_cast<FPropertyAnimatorCoreData*>(this)->PropertyHandler = ControlSubsystem->GetHandler(*this);
		}
	}

	return PropertyHandler;
}

void FPropertyAnimatorCoreData::GetPropertyValuePtrInternal(void* OutValue) const
{
	const FProperty* MemberProperty = GetMemberProperty();
	UObject* Owner = GetOwner();

	if (!Owner || !MemberProperty)
	{
		return;
	}

	// Use getter or directly access property
	MemberProperty->PerformOperationWithGetter(Owner, nullptr, [this, OutValue](const void* InContainer)
	{
		GetLeafPropertyValuePtrInternal(InContainer, OutValue);
	});
}

void FPropertyAnimatorCoreData::SetPropertyValuePtrInternal(const void* InValue) const
{
	const FProperty* MemberProperty = GetMemberProperty();
	UObject* Owner = GetOwner();

	if (!Owner || !MemberProperty)
	{
		return;
	}

	// Use custom setter
	if (HasSetter() && !MemberProperty->HasSetter())
	{
		MemberProperty->PerformOperationWithGetter(Owner, nullptr, [this, MemberProperty, InValue, Owner](const void* InContainer)
		{
			// If we don't have a specified setter function then use the setter found
			UFunction* SetterFunction = SetterFunctionWeak.Get();

			// Lets allocate memory for the setter parameters size
			uint8* NewSetterParamsPtr = static_cast<uint8*>(FMemory::Malloc(SetterFunction->ParmsSize));

			// Copy member property value to setter parameters
			CopyPropertyValue(MemberProperty, InContainer, NewSetterParamsPtr);

			// Set leaf property within setter parameters
			SetLeafPropertyValuePtrInternal(NewSetterParamsPtr, InValue);

			int32 ArgumentCount = 0;
			uint16 Offset = MemberProperty->GetSize();
			for (const FProperty* SetterProperty = SetterFunction->PropertyLink; SetterProperty; SetterProperty = SetterProperty->PropertyLinkNext)
			{
				// is it a parameter of the function
				if (!SetterProperty->IsInContainer(SetterFunction->ParmsSize))
				{
					continue;
				}

				// is it a param property
				if (!SetterProperty->HasAnyPropertyFlags(CPF_Parm))
				{
					continue;
				}

				// is the return value property
				if (SetterProperty->HasAnyPropertyFlags(CPF_ReturnParm))
				{
					continue;
				}

				// This is the value we are trying to set, skip it
				if (ArgumentCount == 0 && SetterProperty->SameType(MemberProperty))
				{
					continue;
				}

				// Allocate new initialized default value
				void* AllocatedDefaultValue = SetterProperty->AllocateAndInitializeValue();

				// Copy default value to setter parameters
				CopyPropertyValue(SetterProperty, AllocatedDefaultValue, NewSetterParamsPtr + Offset);

				// Destroy default value
				SetterProperty->DestroyAndFreeValue(AllocatedDefaultValue);

				ArgumentCount++;
				Offset += SetterProperty->GetSize();
			}

			// Call the ufunction with arguments
			Owner->ProcessEvent(SetterFunction, NewSetterParamsPtr);

			// Free new ufunction setter value
			FMemory::Free(NewSetterParamsPtr);
		});
	}
	else
	{
		// Use regular setter or direct property access
		MemberProperty->PerformOperationWithSetter(Owner, nullptr, [this, InValue](void* InContainer)
		{
			SetLeafPropertyValuePtrInternal(InContainer, InValue);
		});
	}
}

void* FPropertyAnimatorCoreData::ContainerToValuePtr(const void* InContainer, int32 InStartPropertyIndex) const
{
	if (!InContainer)
	{
		return nullptr;
	}

	void* ContainerValue = const_cast<void*>(InContainer);

	for (int32 Idx = InStartPropertyIndex; Idx < ChainProperties.Num(); Idx++)
	{
		ContainerValue = ChainProperties[Idx]->ContainerPtrToValuePtr<uint8>(ContainerValue);
	}

	return ContainerValue;
}

void FPropertyAnimatorCoreData::SetLeafPropertyValuePtrInternal(void* InContainer, const void* InValue) const
{
	if (!InContainer || !InValue)
	{
		return;
	}

	// Start at index 1 since we are not looking inside owner but in our newly copied value
	InContainer = ContainerToValuePtr(InContainer, 1);

	// copy the property value on the leaf property value
	CopyPropertyValue(GetLeafProperty(), InValue, InContainer);
}

void FPropertyAnimatorCoreData::GetLeafPropertyValuePtrInternal(const void* InContainer, void* OutValue) const
{
	if (!InContainer || !OutValue)
	{
		return;
	}

	// Start at index 1 since we are not looking inside owner but in our newly copied value
	InContainer = ContainerToValuePtr(InContainer, 1);

	// copy the leaf property value on out property value
	CopyPropertyValue(GetLeafProperty(), InContainer, OutValue);
}

void FPropertyAnimatorCoreData::CopyPropertyValue(const FProperty* InProperty, const void* InSrc, void* OutDest)
{
	if (!InProperty || !InSrc || !OutDest)
	{
		return;
	}

	if (InProperty->IsA<FBoolProperty>()
		|| InProperty->IsA<FNumericProperty>()
		|| InProperty->IsA<FNameProperty>())
	{
		FMemory::Memcpy(OutDest, InSrc, InProperty->GetSize());
	}
	else
	{
		InProperty->CopyCompleteValue(OutDest, InSrc);
	}
}

FName FPropertyAnimatorCoreData::GetPropertyTypeName(const FProperty* InProperty)
{
	if (InProperty)
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
		{
			return StructProperty->Struct->GetFName();
		}

		return FName(InProperty->GetCPPType());
	}

	return NAME_None;
}

void FPropertyAnimatorCoreData::GeneratePropertyPath()
{
	const UObject* Owner = GetOwner();
	const UObject* StopOuter = GetOwningActor();
	const FString ResolverName = IsResolvable() ? GetPropertyResolver()->GetResolverName().ToString() + TEXT(".") : TEXT("");

	PathHash = ResolverName;
	PathHash += IsValid(Owner) ? Owner->GetPathName(StopOuter) : TEXT("");

	FString DisplayName = ResolverName;
	for (const TFieldPath<FProperty>& ChainProperty : ChainProperties)
	{
		PathHash += TEXT(".") + ChainProperty->GetName();
	}

	GeneratePropertyLocatorPath();
	GeneratePropertyDisplayName();
}

bool FPropertyAnimatorCoreData::FindSetterFunctions()
{
	const UObject* Owner = GetOwner();

	if (!Owner)
	{
		return false;
	}

	const FProperty* MemberProperty = GetMemberProperty();

	if (!MemberProperty)
	{
		return false;
	}

	if (MemberProperty->HasSetter())
	{
		return true;
	}

	if (SetterFunctionWeak.IsValid())
	{
		return true;
	}

	if (bSetterFunctionCached)
	{
		return false;
	}

	FString PropertyName = MemberProperty->GetName();

	// is it a bool property
	if (CastField<FBoolProperty>(MemberProperty))
	{
		PropertyName.RemoveFromStart(TEXT("b"), ESearchCase::CaseSensitive);
	}

	static const TArray<FString> SetterPrefixes = {
		FString("Set"),
		FString("K2_Set"),
		FString("BP_Set")
	};

	for (const FString& Prefix : SetterPrefixes)
	{
		const FName SetterFunctionName = FName(Prefix + PropertyName);

		// Lets see if we have a setter with that name
		UFunction* SetterFunction = Owner->FindFunction(SetterFunctionName);

		// Lets find a matching ufunction setter with matching arguments
		if (!SetterFunction)
		{
			continue;
		}

		// Lets see if this setter arguments match
		bool bValidFunctionSignature = false;
		for (const FProperty* Property = SetterFunction->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			// is it a parameter of the function
			if (!Property->IsInContainer(SetterFunction->ParmsSize))
			{
				continue;
			}

			// is it a param property
			if (!Property->HasAnyPropertyFlags(CPF_Parm))
			{
				continue;
			}

			// is the return value property
			if (Property->HasAnyPropertyFlags(CPF_ReturnParm))
			{
				continue;
			}

			// Property is not compatible with the one we want to set
			// Compatible property should be the first parameter of the setter
			if (!Property->SameType(MemberProperty))
			{
				break;
			}

			bValidFunctionSignature = true;
			break;
		}

		if (bValidFunctionSignature)
		{
			SetterFunctionWeak = SetterFunction;
			break;
		}
	}

	if (!SetterFunctionWeak.IsValid())
	{
		if (UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get())
		{
			SetterFunctionWeak = AnimatorSubsystem->ResolveSetter(MemberProperty->GetFName(), Owner);
		}
	}

	bSetterFunctionCached = true;

	return SetterFunctionWeak.IsValid();
}
