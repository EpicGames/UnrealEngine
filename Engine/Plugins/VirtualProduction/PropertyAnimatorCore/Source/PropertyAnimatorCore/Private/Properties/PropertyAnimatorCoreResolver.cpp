// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/PropertyAnimatorCoreResolver.h"

#include "Presets/PropertyAnimatorCorePresetArchive.h"
#include "Presets/PropertyAnimatorCorePresetBase.h"

bool UPropertyAnimatorCoreResolver::ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue)
{
	if (!InValue->IsObject())
	{
		return false;
	}

	return true;
}

bool UPropertyAnimatorCoreResolver::ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const
{
	const TSharedRef<FPropertyAnimatorCorePresetObjectArchive> TimeSourceArchive = InPreset->GetArchiveImplementation()->CreateObject();
	OutValue = TimeSourceArchive;
	return true;
}
