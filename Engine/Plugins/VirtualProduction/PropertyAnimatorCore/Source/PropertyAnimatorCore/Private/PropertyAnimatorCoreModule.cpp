// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "Logs/PropertyAnimatorCoreLog.h"

DEFINE_LOG_CATEGORY(LogPropertyAnimatorCore)

IMPLEMENT_MODULE(FDefaultModuleImpl, PropertyAnimatorCore)