// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/MovieSceneEvalTemplate.h"
#include "MovieSceneAnimatorExecutionToken.h"
#include "Sequencer/MovieSceneAnimatorTypes.h"
#include "MovieSceneAnimatorEvalTemplate.generated.h"

USTRUCT()
struct FMovieSceneAnimatorEvalTemplate : public FMovieSceneEvalTemplate
{
	GENERATED_BODY()

	FMovieSceneAnimatorEvalTemplate() = default;

	explicit FMovieSceneAnimatorEvalTemplate(const FMovieSceneAnimatorSectionData& InSectionData)
		: SectionData(InSectionData)
	{}

	//~ Begin FMovieSceneEvalTemplate
	virtual UScriptStruct& GetScriptStructImpl() const override
	{
		return *StaticStruct();
	}

	virtual void Evaluate(const FMovieSceneEvaluationOperand& InOperand, const FMovieSceneContext& InContext, const FPersistentEvaluationData& InPersistentData, FMovieSceneExecutionTokens& InExecutionTokens) const override
	{
		FMovieSceneAnimatorExecutionToken ExecutionToken(SectionData);
		InExecutionTokens.Add(MoveTemp(ExecutionToken));
	}
	//~ End FMovieSceneEvalTemplate

private:
	FMovieSceneAnimatorSectionData SectionData;
};
