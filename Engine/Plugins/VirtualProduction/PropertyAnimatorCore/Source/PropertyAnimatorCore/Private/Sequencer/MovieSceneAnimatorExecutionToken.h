// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/MovieSceneExecutionTokens.h"
#include "Sequencer/MovieSceneAnimatorSection.h"
#include "Sequencer/MovieSceneAnimatorTypes.h"
#include "TimeSources/PropertyAnimatorCoreSequencerTimeSource.h"

/** Used to restore state back to previous when outside section */
struct FMovieSceneAnimatorPreAnimatedTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	FMovieSceneAnimatorPreAnimatedTokenProducer()
	{}

	//~ Begin IMovieScenePreAnimatedTokenProducer
	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& InObject) const override
	{
		struct FMovieSceneAnimatorPreAnimatedToken : IMovieScenePreAnimatedToken
		{
			FMovieSceneAnimatorPreAnimatedToken()
			{}

			//~ Begin IMovieScenePreAnimatedToken
			virtual void RestoreState(UObject& InObject, const UE::MovieScene::FRestoreStateParams& InParams) override
			{
				if (UPropertyAnimatorCoreSequencerTimeSource* SequencerTimeSource = Cast<UPropertyAnimatorCoreSequencerTimeSource>(&InObject))
				{
					SequencerTimeSource->OnSequencerTimeEvaluated(TOptional<double>(), TOptional<float>());
				}
			}
			//~ End IMovieScenePreAnimatedToken
		};

		return FMovieSceneAnimatorPreAnimatedToken();
	}
	//~ End IMovieScenePreAnimatedTokenProducer

	static FMovieSceneAnimTypeID GetAnimTypeID()
	{
		return TMovieSceneAnimTypeID<FMovieSceneAnimatorPreAnimatedTokenProducer>();
	}
};

/** Used to evaluate active section */
struct FMovieSceneAnimatorExecutionToken : IMovieSceneExecutionToken
{
	FMovieSceneAnimatorExecutionToken(const FMovieSceneAnimatorSectionData& InSectionData)
		: SectionData(InSectionData)
	{}

	//~ Begin IMovieSceneExecutionToken
	virtual void Execute(const FMovieSceneContext& InContext, const FMovieSceneEvaluationOperand& InOperand, FPersistentEvaluationData& InPersistentData, IMovieScenePlayer& InPlayer) override
	{
		if (InOperand.ObjectBindingID.IsValid() && SectionData.Section)
		{
			double EvaluatedTime = 0;

			if (InContext.IsPreRoll())
			{
				switch (SectionData.EvalTimeMode)
				{
					case EMovieSceneAnimatorEvalTimeMode::Sequence:
					{
						EvaluatedTime = (SectionData.Section->GetInclusiveStartFrame().Value * 1.0) / InContext.GetFrameRate().AsDecimal();
					}
					break;

					case EMovieSceneAnimatorEvalTimeMode::Section:
					{
						EvaluatedTime = 0;
					}
					break;

					case EMovieSceneAnimatorEvalTimeMode::Custom:
					{
						EvaluatedTime = SectionData.CustomStartTime;
					}
					break;
				}
			}
			else if (InContext.IsPostRoll())
			{
				switch (SectionData.EvalTimeMode)
				{
					case EMovieSceneAnimatorEvalTimeMode::Sequence:
					{
						EvaluatedTime = (SectionData.Section->GetExclusiveEndFrame().Value * 1.0) / InContext.GetFrameRate().AsDecimal();
					}
					break;

					case EMovieSceneAnimatorEvalTimeMode::Section:
					{
						const double SectionStartTime = (SectionData.Section->GetInclusiveStartFrame().Value * 1.0) / InContext.GetFrameRate().AsDecimal();
						const double SectionEndTime = (SectionData.Section->GetExclusiveEndFrame().Value * 1.0) / InContext.GetFrameRate().AsDecimal();

						EvaluatedTime = SectionEndTime - SectionStartTime;
					}
					break;

					case EMovieSceneAnimatorEvalTimeMode::Custom:
					{
						EvaluatedTime = SectionData.CustomEndTime;
					}
					break;
				}
			}
			else
			{
				EvaluatedTime = InContext.GetTime().AsDecimal();

				if (SectionData.EvalTimeMode == EMovieSceneAnimatorEvalTimeMode::Section)
				{
					EvaluatedTime -= SectionData.Section->GetInclusiveStartFrame().Value;
				}

				EvaluatedTime /= InContext.GetFrameRate().AsDecimal();

				if (SectionData.EvalTimeMode == EMovieSceneAnimatorEvalTimeMode::Custom)
				{
					const double SectionStartTime = (SectionData.Section->GetInclusiveStartFrame().Value * 1.0) / InContext.GetFrameRate().AsDecimal();
					const double SectionEndTime = (SectionData.Section->GetExclusiveEndFrame().Value * 1.0) / InContext.GetFrameRate().AsDecimal();

					EvaluatedTime = FMath::GetMappedRangeValueClamped(FVector2D(SectionStartTime, SectionEndTime), FVector2D(SectionData.CustomStartTime, SectionData.CustomEndTime), EvaluatedTime);
				}
			}

			const float Magnitude = SectionData.Section->EvaluateEasing(InContext.GetTime());

			for (const TWeakObjectPtr<UObject>& BoundObjectWeak : InPlayer.FindBoundObjects(InOperand))
			{
				UObject* BoundObject = BoundObjectWeak.Get();

				if (!BoundObject)
				{
					continue;
				}

				if (UPropertyAnimatorCoreSequencerTimeSource* SequencerTimeSource = Cast<UPropertyAnimatorCoreSequencerTimeSource>(BoundObject))
				{
					InPlayer.SavePreAnimatedState(*SequencerTimeSource, FMovieSceneAnimatorPreAnimatedTokenProducer::GetAnimTypeID(), FMovieSceneAnimatorPreAnimatedTokenProducer());

					SequencerTimeSource->OnSequencerTimeEvaluated(EvaluatedTime, Magnitude);
				}
			}
		}
	}
	//~ End IMovieSceneExecutionToken

private:
	FMovieSceneAnimatorSectionData SectionData;
};
