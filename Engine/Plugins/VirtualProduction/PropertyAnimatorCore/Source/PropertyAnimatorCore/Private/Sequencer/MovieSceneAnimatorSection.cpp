// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieSceneAnimatorSection.h"

UMovieSceneAnimatorSection::UMovieSceneAnimatorSection()
	: UMovieSceneSection()
{
	bSupportsInfiniteRange = false;
	EvalOptions.CompletionMode = EMovieSceneCompletionMode::RestoreState;
}

void UMovieSceneAnimatorSection::SetEvalTimeMode(EMovieSceneAnimatorEvalTimeMode InMode)
{
	EvalTimeMode = InMode;
}

void UMovieSceneAnimatorSection::SetCustomStartTime(double InTime)
{
	CustomStartTime = InTime;
}

void UMovieSceneAnimatorSection::SetCustomEndTime(double InTime)
{
	CustomEndTime = InTime;
}
