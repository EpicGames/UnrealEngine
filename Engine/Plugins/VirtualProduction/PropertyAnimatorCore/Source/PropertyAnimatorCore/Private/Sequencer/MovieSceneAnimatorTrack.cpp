// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieSceneAnimatorTrack.h"

#include "Evaluation/Blending/MovieSceneBlendType.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "MovieScene.h"
#include "Sequencer/MovieSceneAnimatorEvalTemplate.h"
#include "Sequencer/MovieSceneAnimatorSection.h"

#define LOCTEXT_NAMESPACE "MovieSceneAnimatorTrack"

UMovieSceneAnimatorTrack::UMovieSceneAnimatorTrack()
	: UMovieSceneNameableTrack()
{
	// Needed for easing
	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);
}

bool UMovieSceneAnimatorTrack::SupportsType(TSubclassOf<UMovieSceneSection> InSectionClass) const
{
	return InSectionClass == UMovieSceneAnimatorSection::StaticClass();
}

UMovieSceneSection* UMovieSceneAnimatorTrack::CreateNewSection()
{
	UMovieSceneAnimatorSection* NewSection = NewObject<UMovieSceneAnimatorSection>(this, NAME_None, RF_Transactional);

	if (const UMovieScene* MovieScene = GetTypedOuter<UMovieScene>())
	{
		// Playback range
		NewSection->SetStartFrame(MovieScene->GetPlaybackRange().GetLowerBound());
		NewSection->SetEndFrame(MovieScene->GetPlaybackRange().GetUpperBound());

		// For easing
		NewSection->SetBlendType(EMovieSceneBlendType::Absolute);
		UpdateEasing();

		// Pre/Post roll
		const FFrameRate TickResolution = MovieScene->GetTickResolution();
		NewSection->SetPreRollFrames((1.0 * TickResolution).RoundToFrame().Value);
		NewSection->SetPostRollFrames((1.0 * TickResolution).RoundToFrame().Value);
	}

	return NewSection;
}

void UMovieSceneAnimatorTrack::AddSection(UMovieSceneSection& InSection)
{
	Sections.Add(&InSection);
}

const TArray<UMovieSceneSection*>& UMovieSceneAnimatorTrack::GetAllSections() const
{
	return Sections;
}

bool UMovieSceneAnimatorTrack::HasSection(const UMovieSceneSection& InSection) const
{
	return Sections.Contains(&InSection);
}

bool UMovieSceneAnimatorTrack::IsEmpty() const
{
	return Sections.IsEmpty();
}

void UMovieSceneAnimatorTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}

void UMovieSceneAnimatorTrack::RemoveSection(UMovieSceneSection& InSection)
{
	Sections.Remove(&InSection);
}

void UMovieSceneAnimatorTrack::RemoveSectionAt(int32 InSectionIndex)
{
	Sections.RemoveAt(InSectionIndex);
}

#if WITH_EDITORONLY_DATA
FText UMovieSceneAnimatorTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("MovieSceneAnimatorTrackDefaultName", "Animator Track");
}
#endif

FMovieSceneEvalTemplatePtr UMovieSceneAnimatorTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	const UMovieSceneAnimatorSection* AnimatorSection = Cast<UMovieSceneAnimatorSection>(&InSection);

	if (!AnimatorSection)
	{
		return FMovieSceneEvalTemplatePtr();
	}

	FMovieSceneAnimatorSectionData SectionData;
	SectionData.EvalTimeMode = AnimatorSection->GetEvalTimeMode();
	SectionData.CustomStartTime = AnimatorSection->GetCustomStartTime();
	SectionData.CustomEndTime = AnimatorSection->GetCustomEndTime();
	SectionData.Section = AnimatorSection;

	return FMovieSceneAnimatorEvalTemplate(SectionData);
}

#undef LOCTEXT_NAMESPACE


