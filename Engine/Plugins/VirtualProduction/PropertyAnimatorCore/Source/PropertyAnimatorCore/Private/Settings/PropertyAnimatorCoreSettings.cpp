// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/PropertyAnimatorCoreSettings.h"

#include "Subsystems/PropertyAnimatorCoreSubsystem.h"

UPropertyAnimatorCoreSettings* UPropertyAnimatorCoreSettings::Get()
{
	return GetMutableDefault<UPropertyAnimatorCoreSettings>();
}

UPropertyAnimatorCoreSettings::UPropertyAnimatorCoreSettings()
{
	CategoryName = TEXT("Motion Design");
	SectionName = TEXT("Property Animator Core");

	if (const UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get())
	{
		// Apply default time source
		const TArray<FName> TimeSourceNames = AnimatorSubsystem->GetTimeSourceNames();
		DefaultTimeSourceName = !TimeSourceNames.IsEmpty() ? TimeSourceNames[0] : NAME_None;
	}
}

FName UPropertyAnimatorCoreSettings::GetDefaultTimeSourceName() const
{
	const TArray<FName> TimeSources = GetTimeSourceNames();

	if (TimeSources.Contains(DefaultTimeSourceName))
	{
		return DefaultTimeSourceName;
	}

	return !TimeSources.IsEmpty() ? TimeSources[0] : NAME_None;
}

TArray<FName> UPropertyAnimatorCoreSettings::GetTimeSourceNames() const
{
	TArray<FName> TimeSourceNames;

	if (const UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get())
	{
		TimeSourceNames = AnimatorSubsystem->GetTimeSourceNames();
	}

	return TimeSourceNames;
}

