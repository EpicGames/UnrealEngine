// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "PropertyAnimatorCoreSettings.generated.h"

/** Settings for motion design PropertyAnimatorCore plugin */
UCLASS(Config=Engine, meta=(DisplayName="Property Animator Core"))
class UPropertyAnimatorCoreSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	static UPropertyAnimatorCoreSettings* Get();

	UPropertyAnimatorCoreSettings();

	FName GetDefaultTimeSourceName() const;

protected:
	UFUNCTION()
	TArray<FName> GetTimeSourceNames() const;

	/** The default time source applied on animator by default*/
	UPROPERTY(Config, EditAnywhere, Category="Animator", meta=(GetOptions="GetTimeSourceNames"))
	FName DefaultTimeSourceName = NAME_None;
};
