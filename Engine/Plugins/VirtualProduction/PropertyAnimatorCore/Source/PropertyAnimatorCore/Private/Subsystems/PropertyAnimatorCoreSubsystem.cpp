// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/PropertyAnimatorCoreSubsystem.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Components/PropertyAnimatorCoreComponent.h"
#include "Engine/Engine.h"
#include "GameFramework/Actor.h"
#include "Presets/PropertyAnimatorCorePresetBase.h"
#include "Properties/Converters/PropertyAnimatorCoreConverterBase.h"
#include "Properties/Handlers/PropertyAnimatorCoreHandlerBase.h"
#include "Properties/PropertyAnimatorCoreResolver.h"
#include "TimeSources/PropertyAnimatorCoreTimeSourceBase.h"
#include "TimeSources/PropertyAnimatorCoreWorldTimeSource.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "EngineAnalytics.h"
#include "ScopedTransaction.h"
#endif

#define LOCTEXT_NAMESPACE "PropertyAnimatorCoreSubsystem"

UPropertyAnimatorCoreSubsystem::FOnAnimatorsSetEnabled UPropertyAnimatorCoreSubsystem::OnAnimatorsSetEnabledDelegate;

void UPropertyAnimatorCoreSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	// Register default time source
	RegisterTimeSourceClass(UPropertyAnimatorCoreWorldTimeSource::StaticClass());

	RegisterAnimatorClasses();

	// Register property setter resolver
	RegisterSetterResolver(TEXT("bVisible"), [](const UObject* InOwner)->UFunction*
	{
		return InOwner->FindFunction(TEXT("SetVisibility"));
	});

	RegisterSetterResolver(TEXT("bHidden"), [](const UObject* InOwner)->UFunction*
	{
		return InOwner->FindFunction(TEXT("SetActorHiddenInGame"));
	});

	// Register alias for Rotator properties

	const FString PropertyType = TEXT("Rotator.double.");
	RegisterPropertyAlias(PropertyType + GET_MEMBER_NAME_STRING_CHECKED(FRotator, Roll), TEXT("X"));
	RegisterPropertyAlias(PropertyType + GET_MEMBER_NAME_STRING_CHECKED(FRotator, Pitch), TEXT("Y"));
	RegisterPropertyAlias(PropertyType + GET_MEMBER_NAME_STRING_CHECKED(FRotator, Yaw), TEXT("Z"));

	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.OnFilesLoaded().AddUObject(this, &UPropertyAnimatorCoreSubsystem::OnAssetRegistryFilesLoaded);
	AssetRegistry.OnAssetAdded().AddUObject(this, &UPropertyAnimatorCoreSubsystem::OnAssetRegistryAssetAdded);
	AssetRegistry.OnAssetRemoved().AddUObject(this, &UPropertyAnimatorCoreSubsystem::OnAssetRegistryAssetRemoved);
	AssetRegistry.OnAssetUpdated().AddUObject(this, &UPropertyAnimatorCoreSubsystem::OnAssetRegistryAssetUpdated);
}

void UPropertyAnimatorCoreSubsystem::Deinitialize()
{
	AnimatorsWeak.Empty();
	HandlersWeak.Empty();
	TimeSourcesWeak.Empty();
	ResolversWeak.Empty();
	PresetsWeak.Empty();
	SetterResolvers.Empty();

	if (const FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>("AssetRegistry"))
	{
		IAssetRegistry& AssetRegistry = AssetRegistryModule->Get();
		AssetRegistry.OnFilesLoaded().RemoveAll(this);
		AssetRegistry.OnAssetAdded().RemoveAll(this);
		AssetRegistry.OnAssetRemoved().RemoveAll(this);
		AssetRegistry.OnAssetUpdated().RemoveAll(this);
	}

	Super::Deinitialize();
}

UPropertyAnimatorCoreSubsystem* UPropertyAnimatorCoreSubsystem::Get()
{
	if (GEngine)
	{
		return GEngine->GetEngineSubsystem<UPropertyAnimatorCoreSubsystem>();
	}

	return nullptr;
}

bool UPropertyAnimatorCoreSubsystem::RegisterAnimatorClass(const UClass* InAnimatorClass)
{
	if (!IsValid(InAnimatorClass))
	{
		return false;
	}

	if (!InAnimatorClass->IsChildOf(UPropertyAnimatorCoreBase::StaticClass())
		|| InAnimatorClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists)
#if WITH_EDITOR
		|| InAnimatorClass->HasMetaData(TEXT("Hidden"))
		|| InAnimatorClass->HasMetaData(TEXT("Deprecated"))
#endif
		)
	{
		return false;
	}

	if (IsAnimatorClassRegistered(InAnimatorClass))
	{
		return false;
	}

	if (UPropertyAnimatorCoreBase* CDO = InAnimatorClass->GetDefaultObject<UPropertyAnimatorCoreBase>())
	{
		const TSharedPtr<const FPropertyAnimatorCoreMetadata> Metadata = CDO->GetAnimatorMetadata();

		if (Metadata.IsValid() && !Metadata->Name.IsNone())
		{
			AnimatorsWeak.Add(CDO);

			return true;
		}
	}

	return false;
}

bool UPropertyAnimatorCoreSubsystem::UnregisterAnimatorClass(const UClass* InAnimatorClass)
{
	if (!IsValid(InAnimatorClass))
	{
		return false;
	}

	for (TSet<TWeakObjectPtr<UPropertyAnimatorCoreBase>>::TIterator It(AnimatorsWeak); It; ++It)
	{
		if ((*It)->GetClass() == InAnimatorClass)
		{
			It.RemoveCurrent();
			return true;
		}
	}

	return false;
}

bool UPropertyAnimatorCoreSubsystem::IsAnimatorClassRegistered(const UClass* InAnimatorClass) const
{
	if (!IsValid(InAnimatorClass))
	{
		return false;
	}

	for (const TWeakObjectPtr<UPropertyAnimatorCoreBase>& Animator : AnimatorsWeak)
	{
		if (Animator->GetClass() == InAnimatorClass)
		{
			return true;
		}
	}

	return false;
}

UPropertyAnimatorCoreBase* UPropertyAnimatorCoreSubsystem::GetAnimatorRegistered(const UClass* InAnimatorClass) const
{
	if (!IsValid(InAnimatorClass))
	{
		return nullptr;
	}

	for (const TWeakObjectPtr<UPropertyAnimatorCoreBase>& AnimatorWeak : AnimatorsWeak)
	{
		if (UPropertyAnimatorCoreBase* Animator = AnimatorWeak.Get())
		{
			if (Animator->GetClass() == InAnimatorClass)
			{
				return Animator;
			}
		}
	}

	return nullptr;
}

bool UPropertyAnimatorCoreSubsystem::IsPropertySupported(const FPropertyAnimatorCoreData& InPropertyData, bool bInCheckNestedProperties) const
{
	if (!InPropertyData.IsResolved())
	{
		return false;
	}

	for (const TWeakObjectPtr<UPropertyAnimatorCoreBase>& AnimatorWeak : AnimatorsWeak)
	{
		const UPropertyAnimatorCoreBase* Animator = AnimatorWeak.Get();

		if (!Animator)
		{
			continue;
		}

		if (bInCheckNestedProperties)
		{
			TSet<FPropertyAnimatorCoreData> OutProperties;
			if (Animator->GetPropertiesSupported(InPropertyData, OutProperties))
			{
				return true;
			}
		}
		else if (Animator->HasPropertySupport(InPropertyData))
		{
			return true;
		}
	}

	return false;
}

TSet<UPropertyAnimatorCoreBase*> UPropertyAnimatorCoreSubsystem::GetPropertyLinkedAnimators(const FPropertyAnimatorCoreData& InPropertyData) const
{
	TSet<UPropertyAnimatorCoreBase*> ExistingAnimators = GetExistingAnimators(InPropertyData);

	for (TSet<UPropertyAnimatorCoreBase*>::TIterator It(ExistingAnimators); It; ++It)
	{
		if (!(*It)->IsPropertyLinked(InPropertyData))
		{
			It.RemoveCurrent();
		}
	}

	return ExistingAnimators;
}

TSet<UPropertyAnimatorCoreBase*> UPropertyAnimatorCoreSubsystem::GetExistingAnimators(const FPropertyAnimatorCoreData& InPropertyData) const
{
	TSet<UPropertyAnimatorCoreBase*> ExistingAnimators;

	if (!InPropertyData.IsResolved())
	{
		return ExistingAnimators;
	}

	const AActor* Actor = InPropertyData.GetOwningActor();

	for (UPropertyAnimatorCoreBase* Animator : GetExistingAnimators(Actor))
	{
		TSet<FPropertyAnimatorCoreData> OutProperties;
		if (Animator->GetPropertiesSupported(InPropertyData, OutProperties, /** SearchDepth */3))
		{
			ExistingAnimators.Add(Animator);
		}
	}

	return ExistingAnimators;
}

TSet<UPropertyAnimatorCoreBase*> UPropertyAnimatorCoreSubsystem::GetExistingAnimators(const AActor* InActor) const
{
	TSet<UPropertyAnimatorCoreBase*> ExistingAnimators;

	if (!IsValid(InActor))
	{
		return ExistingAnimators;
	}

	if (const UPropertyAnimatorCoreComponent* PropertyComponent = InActor->FindComponentByClass<UPropertyAnimatorCoreComponent>())
	{
		PropertyComponent->ForEachAnimator([&ExistingAnimators](UPropertyAnimatorCoreBase* InAnimator)->bool
		{
			ExistingAnimators.Add(InAnimator);
			return true;
		});
	}

	return ExistingAnimators;
}

TSet<UPropertyAnimatorCoreBase*> UPropertyAnimatorCoreSubsystem::GetAvailableAnimators(const FPropertyAnimatorCoreData* InPropertyData) const
{
	TSet<UPropertyAnimatorCoreBase*> AvailableAnimators;

	if (InPropertyData && !InPropertyData->IsResolved())
	{
		return AvailableAnimators;
	}

	for (const TWeakObjectPtr<UPropertyAnimatorCoreBase>& Animator : AnimatorsWeak)
	{
		bool bIsPropertySupported = true;
		if (InPropertyData)
		{
			TSet<FPropertyAnimatorCoreData> OutProperties;
			if (!Animator->GetPropertiesSupported(*InPropertyData, OutProperties, /** SearchDepth */3))
			{
				bIsPropertySupported = false;
			}
		}

		if (bIsPropertySupported)
		{
			AvailableAnimators.Add(Animator.Get());
		}
	}

	return AvailableAnimators;
}

TSet<UPropertyAnimatorCoreBase*> UPropertyAnimatorCoreSubsystem::GetAvailableAnimators() const
{
	TSet<UPropertyAnimatorCoreBase*> AvailableAnimators;

	for (const TWeakObjectPtr<UPropertyAnimatorCoreBase>& Animator : AnimatorsWeak)
	{
		AvailableAnimators.Add(Animator.Get());
	}

	return AvailableAnimators;
}

UPropertyAnimatorCoreBase* UPropertyAnimatorCoreSubsystem::CreateAnimator(AActor* InActor, const UClass* InAnimatorClass, UPropertyAnimatorCorePresetBase* InPreset, bool bInTransact) const
{
	if (!IsValid(InActor) || !IsValid(InAnimatorClass))
	{
		return nullptr;
	}

	const TSet<UPropertyAnimatorCoreBase*> NewAnimators = CreateAnimators({InActor}, InAnimatorClass, InPreset, bInTransact);

	if (NewAnimators.IsEmpty())
	{
		return nullptr;
	}

	return NewAnimators.Array()[0];
}

TSet<UPropertyAnimatorCoreBase*> UPropertyAnimatorCoreSubsystem::CreateAnimators(const TSet<AActor*>& InActors, const UClass* InAnimatorClass, UPropertyAnimatorCorePresetBase* InPreset, bool bInTransact) const
{
	return CreateAnimators(InActors, InAnimatorClass, TSet<UPropertyAnimatorCorePresetBase*>{InPreset}, bInTransact);
}

TSet<UPropertyAnimatorCoreBase*> UPropertyAnimatorCoreSubsystem::CreateAnimators(const TSet<AActor*>& InActors, const UClass* InAnimatorClass, const TSet<UPropertyAnimatorCorePresetBase*>& InPresets, bool bInTransact) const
{
	TSet<UPropertyAnimatorCoreBase*> NewAnimators;

	if (InActors.IsEmpty() || !IsValid(InAnimatorClass))
	{
		return NewAnimators;
	}

	const UPropertyAnimatorCoreBase* AnimatorCDO = GetAnimatorRegistered(InAnimatorClass);

	if (!AnimatorCDO)
	{
		return NewAnimators;
	}

	NewAnimators.Reserve(InActors.Num());

#if WITH_EDITOR
	const TSharedPtr<const FPropertyAnimatorCoreMetadata> AnimatorMetadata = AnimatorCDO->GetAnimatorMetadata();
	const FText TransactionText = LOCTEXT("CreateAnimators", "Adding {0} animator to {1} actor(s)");
	const FText AnimatorName = AnimatorMetadata->DisplayName;
	const FText ActorCount = FText::FromString(FString::FromInt(InActors.Num()));

	FScopedTransaction Transaction(FText::Format(TransactionText, AnimatorName, ActorCount), bInTransact && !GIsTransacting);

	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Reserve(2);
		Attributes.Emplace(TEXT("Action"), TEXT("Created"));
		Attributes.Emplace(TEXT("Class"), GetNameSafe(InAnimatorClass));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.PropertyAnimator.Animator"), Attributes);
	}
#endif

	for (AActor* Actor : InActors)
	{
		if (!IsValid(Actor))
		{
			continue;
		}

		UPropertyAnimatorCoreComponent* Component = UPropertyAnimatorCoreComponent::FindOrAdd(Actor);
		if (!Component)
		{
			continue;
		}

#if WITH_EDITOR
		Component->Modify();
#endif

		UPropertyAnimatorCoreBase* NewActorAnimator = Component->AddAnimator(InAnimatorClass);

		if (!NewActorAnimator)
		{
			continue;
		}

#if WITH_EDITOR
		NewActorAnimator->Modify();
#endif

		// Optionally apply presets if any
		for (UPropertyAnimatorCorePresetBase* Preset : InPresets)
		{
			if (Preset && Preset->IsPresetSupported(Actor, NewActorAnimator))
			{
				Preset->ApplyPreset(NewActorAnimator);
			}
		}

		NewAnimators.Add(NewActorAnimator);
	}

	return NewAnimators;
}

TSet<UPropertyAnimatorCoreBase*> UPropertyAnimatorCoreSubsystem::CloneAnimators(const TSet<UPropertyAnimatorCoreBase*>& InAnimators, AActor* InTargetActor, bool bInTransact) const
{
	TSet<UPropertyAnimatorCoreBase*> CopyAnimators;

	if (!IsValid(InTargetActor))
	{
		return CopyAnimators;
	}

#if WITH_EDITOR
	const FText TransactionText = LOCTEXT("CloneAnimators", "Cloning {0} animator(s) on actor %s");
	const FText AnimatorCount = FText::FromString(FString::FromInt(InAnimators.Num()));
	const FText ActorName = FText::FromString(InTargetActor->GetActorNameOrLabel());

	FScopedTransaction Transaction(FText::Format(TransactionText, AnimatorCount, ActorName), bInTransact && !GIsTransacting);
#endif

	UPropertyAnimatorCoreComponent* Component = UPropertyAnimatorCoreComponent::FindOrAdd(InTargetActor);
	if (!Component)
	{
		return CopyAnimators;
	}

#if WITH_EDITOR
	Component->Modify();
#endif

	CopyAnimators.Reserve(InAnimators.Num());

	for (UPropertyAnimatorCoreBase* Animator : InAnimators)
	{
		if (UPropertyAnimatorCoreBase* CopyAnimator = Component->CloneAnimator(Animator))
		{
#if WITH_EDITOR
			CopyAnimator->Modify();
#endif

			CopyAnimators.Add(CopyAnimator);
		}
	}

	return CopyAnimators;
}

bool UPropertyAnimatorCoreSubsystem::RemoveAnimator(UPropertyAnimatorCoreBase* InAnimator, bool bInTransact) const
{
	return RemoveAnimators({InAnimator}, bInTransact);
}

bool UPropertyAnimatorCoreSubsystem::RemoveAnimators(const TSet<UPropertyAnimatorCoreBase*>& InAnimators, bool bInTransact) const
{
	if (InAnimators.IsEmpty())
	{
		return false;
	}

	bool bResult = true;

	for (UPropertyAnimatorCoreBase* Animator : InAnimators)
	{
		if (!IsValid(Animator))
		{
			continue;
		}

		AActor* OwningActor = Animator->GetTypedOuter<AActor>();
		if (!OwningActor)
		{
			continue;
		}

		UPropertyAnimatorCoreComponent* Component = UPropertyAnimatorCoreComponent::FindOrAdd(OwningActor);
		if (!Component)
		{
			continue;
		}

#if WITH_EDITOR
		const FText TransactionText = LOCTEXT("RemoveAnimator", "Removing {0} animator(s)");
		const FText AnimatorCount = FText::FromString(FString::FromInt(InAnimators.Num()));

		FScopedTransaction Transaction(FText::Format(TransactionText, AnimatorCount), bInTransact && !GIsTransacting);

		Component->Modify();
		Animator->Modify();
#endif

		bResult &= Component->RemoveAnimator(Animator);
	}

	return bResult;
}

bool UPropertyAnimatorCoreSubsystem::RemoveAnimatorComponents(const TSet<UPropertyAnimatorCoreComponent*>& InComponents, bool bInTransact) const
{
	if (InComponents.IsEmpty())
	{
		return false;
	}

#if WITH_EDITOR
	const FText TransactionText = LOCTEXT("RemoveAnimatorComponent", "Removing {0} animator component(s)");
	const FText ComponentCount = FText::FromString(FString::FromInt(InComponents.Num()));

	FScopedTransaction Transaction(FText::Format(TransactionText, ComponentCount), bInTransact && !GIsTransacting);
#endif

	for (UPropertyAnimatorCoreComponent* Component : InComponents)
	{
		if (!IsValid(Component))
		{
			continue;
		}

		AActor* OwningActor = Component->GetOwner();

		if (!IsValid(OwningActor))
		{
			continue;
		}

#if WITH_EDITOR
		OwningActor->Modify();
		Component->Modify();
#endif

		Component->DestroyComponent(/** PromoteChildren */false);
	}

	return true;
}

bool UPropertyAnimatorCoreSubsystem::ApplyAnimatorPreset(UPropertyAnimatorCoreBase* InAnimator, UPropertyAnimatorCorePresetBase* InPreset, bool bInTransact)
{
	if (!IsValid(InAnimator) || InAnimator->IsTemplate() || !IsValid(InPreset))
	{
		return false;
	}

	if (!InPreset->IsPresetApplied(InAnimator))
	{
#if WITH_EDITOR
		const FText TransactionText = LOCTEXT("ApplyAnimatorPreset", "Applying {0} preset on {1} animator");
		const FText PresetName = InPreset->GetPresetDisplayName();
		const FText AnimatorName = InAnimator->GetAnimatorMetadata()->DisplayName;

		FScopedTransaction Transaction(FText::Format(TransactionText, PresetName, AnimatorName), bInTransact && !GIsTransacting);

		InAnimator->Modify();
#endif

		return InPreset->ApplyPreset(InAnimator);
	}

	return false;
}

bool UPropertyAnimatorCoreSubsystem::UnapplyAnimatorPreset(UPropertyAnimatorCoreBase* InAnimator, UPropertyAnimatorCorePresetBase* InPreset, bool bInTransact)
{
	if (!IsValid(InAnimator) || InAnimator->IsTemplate() || !IsValid(InPreset))
	{
		return false;
	}

	if (InPreset->IsPresetApplied(InAnimator))
	{
#if WITH_EDITOR
		const FText TransactionText = LOCTEXT("UnapplyAnimatorPreset", "Unapplying {0} preset on {1} animator");
		const FText PresetName = InPreset->GetPresetDisplayName();
		const FText AnimatorName = InAnimator->GetAnimatorMetadata()->DisplayName;

		FScopedTransaction Transaction(FText::Format(TransactionText, PresetName, AnimatorName), bInTransact && !GIsTransacting);

		InAnimator->Modify();
#endif

		return InPreset->UnapplyPreset(InAnimator);
	}

	return false;
}

bool UPropertyAnimatorCoreSubsystem::LinkAnimatorProperty(UPropertyAnimatorCoreBase* InAnimator, FPropertyAnimatorCoreData& InProperty, bool bInTransact)
{
	return LinkAnimatorProperties(InAnimator, {InProperty}, bInTransact);
}

bool UPropertyAnimatorCoreSubsystem::LinkAnimatorProperties(UPropertyAnimatorCoreBase* InAnimator, const TSet<FPropertyAnimatorCoreData>& InProperties, bool bInTransact)
{
	if (!IsValid(InAnimator)
		|| InAnimator->IsTemplate()
		|| InProperties.IsEmpty())
	{
		return false;
	}

	if (!InAnimator->IsPropertiesLinked(InProperties))
	{
#if WITH_EDITOR
		FText TransactionText;
		if (InProperties.Num() == 1)
		{
			const FText PropertyName = FText::FromName(InProperties.Array()[0].GetLeafPropertyName());
			const FText AnimatorName = InAnimator->GetAnimatorMetadata()->DisplayName;

			TransactionText = FText::Format(
				LOCTEXT("LinkAnimatorProperty", "Linking {0} property to {1} animator")
				, PropertyName
				, AnimatorName
			);
		}
		else
		{
			const FText PropertyCount = FText::FromString(FString::FromInt(InProperties.Num()));
			const FText AnimatorName = InAnimator->GetAnimatorMetadata()->DisplayName;

			TransactionText = FText::Format(
				LOCTEXT("LinkAnimatorProperties", "Linking {0} properties to {1} animator")
				, PropertyCount
				, AnimatorName
			);
		}

		FScopedTransaction Transaction(TransactionText, bInTransact && !GIsTransacting);

		InAnimator->Modify();
#endif

		bool bResult = false;
		for (const FPropertyAnimatorCoreData& PropertyData : InProperties)
		{
			bResult |= (InAnimator->LinkProperty(PropertyData) != nullptr);
		}

		return bResult;
	}

	return false;
}

bool UPropertyAnimatorCoreSubsystem::UnlinkAnimatorProperty(UPropertyAnimatorCoreBase* InAnimator, FPropertyAnimatorCoreData& InProperty, bool bInTransact)
{
	return UnlinkAnimatorProperties(InAnimator, {InProperty}, bInTransact);
}

bool UPropertyAnimatorCoreSubsystem::UnlinkAnimatorProperties(UPropertyAnimatorCoreBase* InAnimator, const TSet<FPropertyAnimatorCoreData>& InProperties, bool bInTransact)
{
	if (!IsValid(InAnimator)
		|| InAnimator->IsTemplate()
		|| InProperties.IsEmpty())
	{
		return false;
	}

	if (InAnimator->IsPropertiesLinked(InProperties))
	{
#if WITH_EDITOR
		FText TransactionText;
		if (InProperties.Num() == 1)
		{
			const FText PropertyName = FText::FromName(InProperties.Array()[0].GetLeafPropertyName());
			const FText AnimatorName = InAnimator->GetAnimatorMetadata()->DisplayName;

			TransactionText = FText::Format(
				LOCTEXT("UnlinkAnimatorProperty", "Unlinking {0} property from {1} animator")
				, PropertyName
				, AnimatorName
			);
		}
		else
		{
			const FText PropertyCount = FText::FromString(FString::FromInt(InProperties.Num()));
			const FText AnimatorName = InAnimator->GetAnimatorMetadata()->DisplayName;

			TransactionText = FText::Format(
				LOCTEXT("UnlinkAnimatorProperties", "Unlinking {0} properties from {1} animator")
				, PropertyCount
				, AnimatorName
			);
		}

		FScopedTransaction Transaction(TransactionText, bInTransact && !GIsTransacting);

		InAnimator->Modify();
#endif

		bool bResult = false;
		for (const FPropertyAnimatorCoreData& PropertyData : InProperties)
		{
			bResult |= InAnimator->UnlinkProperty(PropertyData);
		}

		return bResult;
	}

	return false;
}

bool UPropertyAnimatorCoreSubsystem::UnlinkAnimatorProperties(const TSet<UPropertyAnimatorCoreContext*>& InPropertyContexts, bool bInTransact)
{
	bool bResult = false;
	if (InPropertyContexts.IsEmpty())
	{
		return bResult;
	}

#if WITH_EDITOR
	FScopedTransaction Transaction(
		FText::Format(LOCTEXT("UnlinkingAnimatorPropertyContexts", "Unlinking {0} properties from their animators"),
			FText::FromString(FString::FromInt(InPropertyContexts.Num())))
		, bInTransact && !GIsTransacting);
#endif

	for (const UPropertyAnimatorCoreContext* PropertyContext : InPropertyContexts)
	{
		if (PropertyContext)
		{
			if (UPropertyAnimatorCoreBase* Animator = PropertyContext->GetAnimator())
			{
#if WITH_EDITOR
				Animator->Modify();
#endif
				bResult |= Animator->UnlinkProperty(PropertyContext->GetAnimatedProperty());
			}
		}
	}

#if WITH_EDITOR
	if (!bResult)
	{
		Transaction.Cancel();		
	}
#endif

	return bResult;
}

void UPropertyAnimatorCoreSubsystem::SetAnimatorPropertiesEnabled(const TSet<UPropertyAnimatorCoreContext*>& InPropertyContexts, bool bInEnabled, bool bInTransact)
{
	if (InPropertyContexts.IsEmpty())
	{
		return;
	}

#if WITH_EDITOR
	const FText TransactionText = bInEnabled
		? LOCTEXT("SetAnimatorPropertiesEnabled", "{0} animator properties enabled")
		: LOCTEXT("SetAnimatorPropertiesDisabled", "{0} animator properties disabled");
	
	FScopedTransaction Transaction(
		FText::Format(TransactionText,
			FText::FromString(FString::FromInt(InPropertyContexts.Num())))
		, bInTransact && !GIsTransacting);
#endif

	bool bResult = false;
	for (UPropertyAnimatorCoreContext* PropertyContext : InPropertyContexts)
	{
		if (PropertyContext && PropertyContext->IsAnimated() != bInEnabled)
		{
#if WITH_EDITOR
			PropertyContext->Modify();
#endif

			PropertyContext->SetAnimated(bInEnabled);
			
			bResult |= true;
		}
	}

#if WITH_EDITOR
	if (!bResult)
	{
		Transaction.Cancel();		
	}
#endif
}

bool UPropertyAnimatorCoreSubsystem::RegisterHandlerClass(const UClass* InHandlerClass)
{
	if (!IsValid(InHandlerClass))
	{
		return false;
	}

	if (!InHandlerClass->IsChildOf(UPropertyAnimatorCoreHandlerBase::StaticClass())
		|| InHandlerClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
	{
		return false;
	}

	if (IsHandlerClassRegistered(InHandlerClass))
	{
		return false;
	}

	if (UPropertyAnimatorCoreHandlerBase* CDO = InHandlerClass->GetDefaultObject<UPropertyAnimatorCoreHandlerBase>())
	{
		HandlersWeak.Add(CDO);

		return true;
	}

	return false;
}

bool UPropertyAnimatorCoreSubsystem::UnregisterHandlerClass(const UClass* InHandlerClass)
{
	if (!IsValid(InHandlerClass))
	{
		return false;
	}

	for (TSet<TWeakObjectPtr<UPropertyAnimatorCoreHandlerBase>>::TIterator It(HandlersWeak); It; ++It)
	{
		if (It->Get()->GetClass() == InHandlerClass)
		{
			It.RemoveCurrent();
			return true;
		}
	}

	return false;
}

bool UPropertyAnimatorCoreSubsystem::IsHandlerClassRegistered(const UClass* InHandlerClass) const
{
	for (const TWeakObjectPtr<UPropertyAnimatorCoreHandlerBase>& HandlerWeakPair : HandlersWeak)
	{
		if (HandlerWeakPair->GetClass() == InHandlerClass)
		{
			return true;
		}
	}

	return false;
}

bool UPropertyAnimatorCoreSubsystem::RegisterResolverClass(const UClass* InResolverClass)
{
	if (!IsValid(InResolverClass))
	{
		return false;
	}

	if (!InResolverClass->IsChildOf(UPropertyAnimatorCoreResolver::StaticClass())
		|| InResolverClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
	{
		return false;
	}

	if (IsResolverClassRegistered(InResolverClass))
	{
		return false;
	}

	if (UPropertyAnimatorCoreResolver* CDO = InResolverClass->GetDefaultObject<UPropertyAnimatorCoreResolver>())
	{
		ResolversWeak.Add(CDO);

		return true;
	}

	return false;
}

bool UPropertyAnimatorCoreSubsystem::UnregisterResolverClass(const UClass* InResolverClass)
{
	if (!IsValid(InResolverClass))
	{
		return false;
	}

	for (TSet<TWeakObjectPtr<UPropertyAnimatorCoreResolver>>::TIterator It(ResolversWeak); It; ++It)
	{
		if (It->Get()->GetClass() == InResolverClass)
		{
			It.RemoveCurrent();
			return true;
		}
	}

	return false;
}

UPropertyAnimatorCoreResolver* UPropertyAnimatorCoreSubsystem::FindResolverByName(FName InResolverName)
{
	for (const TWeakObjectPtr<UPropertyAnimatorCoreResolver>& ResolverWeakPair : ResolversWeak)
	{
		UPropertyAnimatorCoreResolver* Resolver = ResolverWeakPair.Get();

		if (Resolver && Resolver->GetResolverName().IsEqual(InResolverName))
		{
			return Resolver;
		}
	}

	return nullptr;
}

UPropertyAnimatorCoreResolver* UPropertyAnimatorCoreSubsystem::FindResolverByClass(const UClass* InResolverClass)
{
	for (const TWeakObjectPtr<UPropertyAnimatorCoreResolver>& ResolverWeakPair : ResolversWeak)
	{
		UPropertyAnimatorCoreResolver* Resolver = ResolverWeakPair.Get();

		if (Resolver && Resolver->GetClass() == InResolverClass)
		{
			return Resolver;
		}
	}

	return nullptr;
}

bool UPropertyAnimatorCoreSubsystem::IsResolverClassRegistered(const UClass* InResolverClass) const
{
	for (const TWeakObjectPtr<UPropertyAnimatorCoreResolver>& ResolverWeakPair : ResolversWeak)
	{
		if (ResolverWeakPair->GetClass() == InResolverClass)
		{
			return true;
		}
	}

	return false;
}

bool UPropertyAnimatorCoreSubsystem::RegisterTimeSourceClass(UClass* InTimeSourceClass)
{
	if (!IsValid(InTimeSourceClass))
	{
		return false;
	}

	if (!InTimeSourceClass->IsChildOf(UPropertyAnimatorCoreTimeSourceBase::StaticClass())
		|| InTimeSourceClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
	{
		return false;
	}

	if (IsTimeSourceClassRegistered(InTimeSourceClass))
	{
		return false;
	}

	UPropertyAnimatorCoreTimeSourceBase* TimeSourceCDO = InTimeSourceClass->GetDefaultObject<UPropertyAnimatorCoreTimeSourceBase>();
	if (!TimeSourceCDO)
	{
		return false;
	}

	const FName TimeSourceName = TimeSourceCDO->GetTimeSourceName();
	if (TimeSourceName.IsNone())
	{
		return false;
	}

	TimeSourcesWeak.Add(TimeSourceCDO);
	TimeSourceCDO->OnTimeSourceRegistered();

	return true;
}

bool UPropertyAnimatorCoreSubsystem::UnregisterTimeSourceClass(UClass* InTimeSourceClass)
{
	if (!IsValid(InTimeSourceClass))
	{
		return false;
	}

	for (TSet<TWeakObjectPtr<UPropertyAnimatorCoreTimeSourceBase>>::TIterator It(TimeSourcesWeak); It; ++It)
	{
		if (It->Get()->GetClass() == InTimeSourceClass)
		{
			It.RemoveCurrent();
			return true;
		}
	}

	return false;
}

bool UPropertyAnimatorCoreSubsystem::IsTimeSourceClassRegistered(UClass* InTimeSourceClass) const
{
	if (!IsValid(InTimeSourceClass))
	{
		return false;
	}

	for (const TWeakObjectPtr<UPropertyAnimatorCoreTimeSourceBase>& TimeSourceWeak : TimeSourcesWeak)
	{
		if (const UPropertyAnimatorCoreTimeSourceBase* TimeSource = TimeSourceWeak.Get())
		{
			if (TimeSource->GetClass() == InTimeSourceClass)
			{
				return true;
			}
		}
	}

	return false;
}

TArray<FName> UPropertyAnimatorCoreSubsystem::GetTimeSourceNames() const
{
	TArray<FName> TimeSourceNames;
	TimeSourceNames.Reserve(TimeSourcesWeak.Num());

	for (const TWeakObjectPtr<UPropertyAnimatorCoreTimeSourceBase>& TimeSourceWeak : TimeSourcesWeak)
	{
		if (const UPropertyAnimatorCoreTimeSourceBase* TimeSource = TimeSourceWeak.Get())
		{
			TimeSourceNames.Add(TimeSource->GetTimeSourceName());
		}
	}

	return TimeSourceNames;
}

TArray<UPropertyAnimatorCoreTimeSourceBase*> UPropertyAnimatorCoreSubsystem::GetTimeSources() const
{
	TArray<UPropertyAnimatorCoreTimeSourceBase*> TimeSources;

	Algo::TransformIf(
		TimeSourcesWeak
		, TimeSources
		, [](const TWeakObjectPtr<UPropertyAnimatorCoreTimeSourceBase>& InTimeSourceWeak)
		{
			return InTimeSourceWeak.IsValid();
		}
		, [](const TWeakObjectPtr<UPropertyAnimatorCoreTimeSourceBase>& InTimeSourceWeak)
		{
			return InTimeSourceWeak.Get();
		}
	);

	return TimeSources;
}

UPropertyAnimatorCoreTimeSourceBase* UPropertyAnimatorCoreSubsystem::GetTimeSource(FName InTimeSourceName) const
{
	if (InTimeSourceName.IsNone())
	{
		return nullptr;
	}

	for (const TWeakObjectPtr<UPropertyAnimatorCoreTimeSourceBase>& TimeSourceWeak : TimeSourcesWeak)
	{
		if (UPropertyAnimatorCoreTimeSourceBase* TimeSource = TimeSourceWeak.Get())
		{
			if (TimeSource->GetTimeSourceName() == InTimeSourceName)
			{
				return TimeSource;
			}
		}
	}

	return nullptr;
}

UPropertyAnimatorCoreTimeSourceBase* UPropertyAnimatorCoreSubsystem::CreateNewTimeSource(FName InTimeSourceName, UObject* InOwner)
{
	if (!IsValid(InOwner) || InTimeSourceName.IsNone())
	{
		return nullptr;
	}

	const UPropertyAnimatorCoreTimeSourceBase* TimeSource = GetTimeSource(InTimeSourceName);

	if (!TimeSource)
	{
		return nullptr;
	}

	// Here unique name needs to be provided
	const UClass* TimeSourceClass = TimeSource->GetClass();
	const FName UniqueObjectName = MakeUniqueObjectName(InOwner, TimeSourceClass, InTimeSourceName);
	return NewObject<UPropertyAnimatorCoreTimeSourceBase>(InOwner, TimeSourceClass, UniqueObjectName);
}

bool UPropertyAnimatorCoreSubsystem::RegisterPresetClass(const UClass* InPresetClass)
{
	if (!IsValid(InPresetClass))
	{
		return false;
	}

	if (!InPresetClass->IsChildOf(UPropertyAnimatorCorePresetBase::StaticClass())
		|| InPresetClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
	{
		return false;
	}

	if (IsPresetClassRegistered(InPresetClass))
	{
		return false;
	}

	if (UPropertyAnimatorCorePresetBase* CDO = InPresetClass->GetDefaultObject<UPropertyAnimatorCorePresetBase>())
	{
		if (CDO->LoadPreset())
		{
			PresetsWeak.Add(CDO);
			CDO->OnPresetRegistered();
		}

		return true;
	}

	return false;
}

bool UPropertyAnimatorCoreSubsystem::UnregisterPresetClass(const UClass* InPresetClass)
{
	if (!IsValid(InPresetClass))
	{
		return false;
	}

	for (TSet<TWeakObjectPtr<UPropertyAnimatorCorePresetBase>>::TIterator It(PresetsWeak); It; ++It)
	{
		if ((*It)->GetClass() == InPresetClass)
		{
			It.RemoveCurrent();
			return true;
		}
	}

	return false;
}

bool UPropertyAnimatorCoreSubsystem::IsPresetClassRegistered(const UClass* InPresetClass) const
{
	if (!IsValid(InPresetClass))
	{
		return false;
	}

	for (const TWeakObjectPtr<UPropertyAnimatorCorePresetBase>& Preset : PresetsWeak)
	{
		if (Preset->GetClass() == InPresetClass)
		{
			return true;
		}
	}

	return false;
}

TSet<UPropertyAnimatorCorePresetBase*> UPropertyAnimatorCoreSubsystem::GetAvailablePresets(TSubclassOf<UPropertyAnimatorCorePresetBase> InPresetClass) const
{
	TSet<UPropertyAnimatorCorePresetBase*> AvailablePresets;
	AvailablePresets.Reserve(PresetsWeak.Num());

	Algo::TransformIf(
		PresetsWeak
		, AvailablePresets
		, [&InPresetClass](const TWeakObjectPtr<UPropertyAnimatorCorePresetBase>& InPresetWeak)
		{
			return InPresetWeak.IsValid() && InPresetWeak->IsA(InPresetClass);
		}
		, [](const TWeakObjectPtr<UPropertyAnimatorCorePresetBase>& InPresetWeak)
		{
			return InPresetWeak.Get();
		}
	);

	return AvailablePresets;
}

TSet<UPropertyAnimatorCorePresetBase*> UPropertyAnimatorCoreSubsystem::GetSupportedPresets(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSubclassOf<UPropertyAnimatorCorePresetBase> InPresetClass) const
{
	TSet<UPropertyAnimatorCorePresetBase*> SupportedPresets;

	for (const TWeakObjectPtr<UPropertyAnimatorCorePresetBase>& PresetWeak : PresetsWeak)
	{
		UPropertyAnimatorCorePresetBase* Preset = PresetWeak.Get();

		if (!Preset || !Preset->IsA(InPresetClass))
		{
			continue;
		}

		if (!Preset->IsPresetSupported(InActor, InAnimator))
		{
			continue;
		}

		SupportedPresets.Add(Preset);
	}

	return SupportedPresets;
}

bool UPropertyAnimatorCoreSubsystem::RegisterConverterClass(const UClass* InConverterClass)
{
	if (!IsValid(InConverterClass))
	{
		return false;
	}

	if (!InConverterClass->IsChildOf(UPropertyAnimatorCoreConverterBase::StaticClass())
		|| InConverterClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
	{
		return false;
	}

	if (IsConverterClassRegistered(InConverterClass))
	{
		return false;
	}

	if (UPropertyAnimatorCoreConverterBase* CDO = InConverterClass->GetDefaultObject<UPropertyAnimatorCoreConverterBase>())
	{
		ConvertersWeak.Add(CDO);

		return true;
	}

	return false;
}

bool UPropertyAnimatorCoreSubsystem::UnregisterConverterClass(const UClass* InConverterClass)
{
	if (!IsValid(InConverterClass))
	{
		return false;
	}

	for (TSet<TWeakObjectPtr<UPropertyAnimatorCoreConverterBase>>::TIterator It(ConvertersWeak); It; ++It)
	{
		if ((*It)->GetClass() == InConverterClass)
		{
			It.RemoveCurrent();
			return true;
		}
	}

	return false;
}

bool UPropertyAnimatorCoreSubsystem::IsConverterClassRegistered(const UClass* InConverterClass)
{
	if (!IsValid(InConverterClass))
	{
		return false;
	}

	for (const TWeakObjectPtr<UPropertyAnimatorCoreConverterBase>& ConverterWeak : ConvertersWeak)
	{
		if (const UPropertyAnimatorCoreConverterBase* Converter = ConverterWeak.Get())
		{
			if (Converter->GetClass() == InConverterClass)
			{
				return true;
			}
		}
	}

	return false;
}

bool UPropertyAnimatorCoreSubsystem::IsConversionSupported(const FPropertyBagPropertyDesc& InFromProperty, const FPropertyBagPropertyDesc& InToProperty)
{
	for (const TWeakObjectPtr<UPropertyAnimatorCoreConverterBase>& ConverterWeak : ConvertersWeak)
	{
		if (const UPropertyAnimatorCoreConverterBase* Converter = ConverterWeak.Get())
		{
			if (Converter->IsConversionSupported(InFromProperty, InToProperty))
			{
				return true;
			}
		}
	}

	return false;
}

TSet<UPropertyAnimatorCoreConverterBase*> UPropertyAnimatorCoreSubsystem::GetSupportedConverters(const FPropertyBagPropertyDesc& InFromProperty, const FPropertyBagPropertyDesc& InToProperty) const
{
	TSet<UPropertyAnimatorCoreConverterBase*> SupportedConverters;

	for (const TWeakObjectPtr<UPropertyAnimatorCoreConverterBase>& ConverterWeak : ConvertersWeak)
	{
		if (UPropertyAnimatorCoreConverterBase* Converter = ConverterWeak.Get())
		{
			if (Converter->IsConversionSupported(InFromProperty, InToProperty))
			{
				SupportedConverters.Add(Converter);
			}
		}
	}

	return SupportedConverters;
}

bool UPropertyAnimatorCoreSubsystem::RegisterPropertyAlias(const FString& InPropertyIdentifier, const FString& InAliasPropertyName)
{
	if (InPropertyIdentifier.IsEmpty() || InAliasPropertyName.IsEmpty())
	{
		return false;
	}

	PropertyAliases.Add(InPropertyIdentifier, InAliasPropertyName);

	return true;
}

bool UPropertyAnimatorCoreSubsystem::UnregisterPropertyAlias(const FString& InPropertyIdentifier)
{
	return PropertyAliases.Remove(InPropertyIdentifier) > 0;
}

FString UPropertyAnimatorCoreSubsystem::FindPropertyAlias(const FString& InPropertyIdentifier) const
{
	FString Alias;

	if (!InPropertyIdentifier.IsEmpty())
	{
		if (const FString* const AliasPtr = PropertyAliases.Find(InPropertyIdentifier))
		{
			Alias = *AliasPtr;
		}
	}

	return Alias;
}

void UPropertyAnimatorCoreSubsystem::SetActorAnimatorsEnabled(const TSet<AActor*>& InActors, bool bInEnabled, bool bInTransact)
{
	if (InActors.IsEmpty())
	{
		return;
	}

#if WITH_EDITOR
	const FText TransactionText = bInEnabled
		? LOCTEXT("SetActorAnimatorsEnabled", "Actors Animators Enabled")
		: LOCTEXT("SetActorAnimatorsDisabled", "Actors Animators Disabled");

	FScopedTransaction Transaction(TransactionText, bInTransact && !GIsTransacting);
#endif

	for (const AActor* Actor : InActors)
	{
		if (!IsValid(Actor))
		{
			continue;
		}

		UPropertyAnimatorCoreComponent* AnimatorComponent = Actor->FindComponentByClass<UPropertyAnimatorCoreComponent>();

		if (!IsValid(AnimatorComponent))
		{
			continue;
		}

#if WITH_EDITOR
		AnimatorComponent->Modify();
#endif

		AnimatorComponent->SetAnimatorsEnabled(bInEnabled);
	}
}

void UPropertyAnimatorCoreSubsystem::SetLevelAnimatorsEnabled(const UWorld* InWorld, bool bInEnabled, bool bInTransact)
{
	if (!IsValid(InWorld))
	{
		return;
	}

#if WITH_EDITOR
	const FText TransactionText = bInEnabled
		? LOCTEXT("SetLevelAnimatorsEnabled", "Level Animators Enabled")
		: LOCTEXT("SetLevelAnimatorsDisabled", "Level Animators Disabled");

	FScopedTransaction Transaction(TransactionText, bInTransact && !GIsTransacting);
#endif

	OnAnimatorsSetEnabledDelegate.Broadcast(InWorld, bInEnabled, bInTransact);
}

void UPropertyAnimatorCoreSubsystem::SetAnimatorsEnabled(const TSet<UPropertyAnimatorCoreBase*>& InAnimators, bool bInEnabled, bool bInTransact)
{
	if (InAnimators.IsEmpty())
	{
		return;
	}

#if WITH_EDITOR
	const FText TransactionText = bInEnabled
		? LOCTEXT("SetAnimatorsEnabled", "{0} Animators Enabled")
		: LOCTEXT("SetAnimatorsDisabled", "{0} Animators Disabled");

	FScopedTransaction Transaction(FText::Format(TransactionText, FText::FromString(FString::FromInt(InAnimators.Num()))), bInTransact && !GIsTransacting);
#endif

	for (UPropertyAnimatorCoreBase* Animator : InAnimators)
	{
		if (!IsValid(Animator))
		{
			continue;
		}

#if WITH_EDITOR
		Animator->Modify();
#endif

		Animator->SetAnimatorEnabled(bInEnabled);
	}
}

bool UPropertyAnimatorCoreSubsystem::RegisterSetterResolver(FName InPropertyName, TFunction<UFunction*(const UObject*)>&& InFunction)
{
	if (InPropertyName.IsNone())
	{
		return false;
	}

	SetterResolvers.Emplace(InPropertyName, InFunction);

	return true;
}

bool UPropertyAnimatorCoreSubsystem::UnregisterSetterResolver(FName InPropertyName)
{
	return SetterResolvers.Remove(InPropertyName) > 0;
}

bool UPropertyAnimatorCoreSubsystem::IsSetterResolverRegistered(FName InPropertyName) const
{
	return SetterResolvers.Contains(InPropertyName);
}

UFunction* UPropertyAnimatorCoreSubsystem::ResolveSetter(FName InPropertyName, const UObject* InOwner)
{
	if (!IsValid(InOwner))
	{
		return nullptr;
	}

	if (const TFunction<UFunction*(const UObject*)>* SetterResolver = SetterResolvers.Find(InPropertyName))
	{
		return (*SetterResolver)(InOwner);
	}

	return nullptr;
}

UPropertyAnimatorCoreHandlerBase* UPropertyAnimatorCoreSubsystem::GetHandler(const FPropertyAnimatorCoreData& InPropertyData) const
{
	if (!InPropertyData.IsResolved())
	{
		return nullptr;
	}

	for (const TWeakObjectPtr<UPropertyAnimatorCoreHandlerBase>& HandlerWeak : HandlersWeak)
	{
		UPropertyAnimatorCoreHandlerBase* Handler = HandlerWeak.Get();
		if (Handler && Handler->IsPropertySupported(InPropertyData))
		{
			return Handler;
		}
	}

	return nullptr;
}

void UPropertyAnimatorCoreSubsystem::RegisterAnimatorClasses()
{
	for (UClass* const Class : TObjectRange<UClass>())
	{
		RegisterAnimatorClass(Class);
		RegisterHandlerClass(Class);
		RegisterResolverClass(Class);
		RegisterTimeSourceClass(Class);
		RegisterPresetClass(Class);
		RegisterConverterClass(Class);
	}
}

void UPropertyAnimatorCoreSubsystem::OnAssetRegistryFilesLoaded()
{
	bFilesLoaded = true;

	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	TArray<FAssetData> Assets;
	AssetRegistryModule.Get().GetAssetsByClass(UPropertyAnimatorCorePresetBase::StaticClass()->GetClassPathName(), Assets, /** Subclass */true);

	for (const FAssetData& Asset : Assets)
	{
		RegisterPresetAsset(Asset);
	}
}

void UPropertyAnimatorCoreSubsystem::OnAssetRegistryAssetAdded(const FAssetData& InAssetData)
{
	if (bFilesLoaded)
	{
		RegisterPresetAsset(InAssetData);
	}
}

void UPropertyAnimatorCoreSubsystem::OnAssetRegistryAssetRemoved(const FAssetData& InAssetData)
{
	UnregisterPresetAsset(InAssetData);
}

void UPropertyAnimatorCoreSubsystem::OnAssetRegistryAssetUpdated(const FAssetData& InAssetData)
{
	UnregisterPresetAsset(InAssetData);
	RegisterPresetAsset(InAssetData);
}

void UPropertyAnimatorCoreSubsystem::RegisterPresetAsset(const FAssetData& InAssetData)
{
	if (const UClass* Class = InAssetData.GetClass(EResolveClass::Yes))
	{
		if (!Class->IsChildOf<UPropertyAnimatorCorePresetBase>())
		{
			return;
		}

		if (UPropertyAnimatorCorePresetBase* Preset = Cast<UPropertyAnimatorCorePresetBase>(InAssetData.GetAsset()))
		{
			if (Preset->LoadPreset())
			{
				PresetsWeak.Add(Preset);
				Preset->OnPresetRegistered();
			}
		}
	}
}

void UPropertyAnimatorCoreSubsystem::UnregisterPresetAsset(const FAssetData& InAssetData)
{
	if (const UClass* Class = InAssetData.GetClass(EResolveClass::Yes))
	{
		if (!Class->IsChildOf<UPropertyAnimatorCorePresetBase>())
		{
			return;
		}

		if (UPropertyAnimatorCorePresetBase* Preset = Cast<UPropertyAnimatorCorePresetBase>(InAssetData.GetAsset()))
		{
			if (PresetsWeak.Remove(Preset) > 0)
			{
				Preset->OnPresetUnregistered();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
