// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimeSources/PropertyAnimatorCoreManualTimeSource.h"

#include "Misc/App.h"
#include "Presets/PropertyAnimatorCorePresetArchive.h"

FName UPropertyAnimatorCoreManualTimeSource::GetCustomTimePropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreManualTimeSource, CustomTime);
}

void UPropertyAnimatorCoreManualTimeSource::SetCustomTime(double InTime)
{
	if (FMath::IsNearlyEqual(InTime, CustomTime))
	{
		return;
	}

	CustomTime = InTime;
}

void UPropertyAnimatorCoreManualTimeSource::SetPlaybackState(EPropertyAnimatorCoreManualStatus InState)
{
	if (PlaybackState == InState)
	{
		return;
	}

	PlaybackState = InState;
	OnStateChanged();
}

bool UPropertyAnimatorCoreManualTimeSource::UpdateEvaluationData(FPropertyAnimatorCoreTimeSourceEvaluationData& OutData)
{
	/*
	 * Don't use world delta time to avoid time dilation,
	 * get the app to use raw time between frames and increment when this time source is enabled
	 */
	if (ActiveStatus == EPropertyAnimatorCoreManualStatus::PlayingForward)
	{
		CustomTime += FApp::GetDeltaTime();
	}
	else if (ActiveStatus == EPropertyAnimatorCoreManualStatus::PlayingBackward)
	{
		CustomTime -= FApp::GetDeltaTime();
	}

	OutData.TimeElapsed = CustomTime;

	return ActiveStatus != EPropertyAnimatorCoreManualStatus::Stopped;
}

void UPropertyAnimatorCoreManualTimeSource::OnTimeSourceActive()
{
	Super::OnTimeSourceActive();

	PlaybackState = EPropertyAnimatorCoreManualStatus::Paused;
	ActiveStatus = PlaybackState;
}

void UPropertyAnimatorCoreManualTimeSource::OnTimeSourceInactive()
{
	Super::OnTimeSourceInactive();

	Stop();
}

bool UPropertyAnimatorCoreManualTimeSource::ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue)
{
	if (Super::ImportPreset(InPreset, InValue) && InValue->IsObject())
	{
		TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> ObjectArchive = InValue->AsMutableObject();

		double CustomTimeValue = CustomTime;
		ObjectArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreManualTimeSource, CustomTime), CustomTimeValue);
		SetCustomTime(CustomTimeValue);

		return true;
	}

	return false;
}

bool UPropertyAnimatorCoreManualTimeSource::ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const
{
	if (Super::ExportPreset(InPreset, OutValue) && OutValue->IsObject())
	{
		TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> ObjectArchive = OutValue->AsMutableObject();

		ObjectArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreManualTimeSource, CustomTime), CustomTime);

		return true;
	}

	return false;
}

void UPropertyAnimatorCoreManualTimeSource::Play(bool bInForward)
{
	// Allow change from playing forward to backward
	if (!IsPlaying()
		|| (bInForward && ActiveStatus == EPropertyAnimatorCoreManualStatus::PlayingBackward)
		|| (!bInForward && ActiveStatus == EPropertyAnimatorCoreManualStatus::PlayingForward))
	{
		ActiveStatus = bInForward ? EPropertyAnimatorCoreManualStatus::PlayingForward : EPropertyAnimatorCoreManualStatus::PlayingBackward;
	}
}

void UPropertyAnimatorCoreManualTimeSource::Pause()
{
	if (!IsPlaying())
	{
		return;
	}

	ActiveStatus = EPropertyAnimatorCoreManualStatus::Paused;
}

void UPropertyAnimatorCoreManualTimeSource::Stop()
{
	if (ActiveStatus == EPropertyAnimatorCoreManualStatus::Stopped)
	{
		return;
	}

	Pause();
	CustomTime = 0;
	ActiveStatus = EPropertyAnimatorCoreManualStatus::Stopped;
}

EPropertyAnimatorCoreManualStatus UPropertyAnimatorCoreManualTimeSource::GetPlaybackStatus() const
{
	return ActiveStatus;
}

bool UPropertyAnimatorCoreManualTimeSource::IsPlaying() const
{
	return ActiveStatus == EPropertyAnimatorCoreManualStatus::PlayingBackward || ActiveStatus == EPropertyAnimatorCoreManualStatus::PlayingForward;
}

#if WITH_EDITOR
void UPropertyAnimatorCoreManualTimeSource::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreManualTimeSource, PlaybackState))
	{
		OnStateChanged();
	}
}
#endif

void UPropertyAnimatorCoreManualTimeSource::OnStateChanged()
{
	switch (PlaybackState)
	{
	case EPropertyAnimatorCoreManualStatus::Stopped:
		Stop();
		break;
	case EPropertyAnimatorCoreManualStatus::Paused:
		Pause();
		break;
	case EPropertyAnimatorCoreManualStatus::PlayingForward:
		Play(/** Forward */true);
		break;
	case EPropertyAnimatorCoreManualStatus::PlayingBackward:
		Play(/** Forward */false);
		break;
	}
}
