// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimeSources/PropertyAnimatorCoreSequencerTimeSource.h"

bool UPropertyAnimatorCoreSequencerTimeSource::UpdateEvaluationData(FPropertyAnimatorCoreTimeSourceEvaluationData& OutData)
{
	if (!EvalResult.bEvalValid)
	{
		return false;
	}

	OutData.TimeElapsed = EvalResult.EvalTime;
	OutData.Magnitude = EvalResult.EvalMagnitude;

	return true;
}

void UPropertyAnimatorCoreSequencerTimeSource::OnSequencerTimeEvaluated(const TOptional<double>& InTimeEval, const TOptional<float>& InMagnitudeEval)
{
	EvalResult.bEvalValid = InTimeEval.IsSet() && InMagnitudeEval.IsSet();

	if (EvalResult.bEvalValid)
	{
		EvalResult.EvalTime = InTimeEval.GetValue();
		EvalResult.EvalMagnitude = InMagnitudeEval.GetValue();
	}
}
