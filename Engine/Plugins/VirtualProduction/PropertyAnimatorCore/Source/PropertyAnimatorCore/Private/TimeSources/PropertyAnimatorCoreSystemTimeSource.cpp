// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimeSources/PropertyAnimatorCoreSystemTimeSource.h"

#include "Internationalization/Regex.h"
#include "Misc/DateTime.h"
#include "Presets/PropertyAnimatorCorePresetArchive.h"

bool UPropertyAnimatorCoreSystemTimeSource::UpdateEvaluationData(FPropertyAnimatorCoreTimeSourceEvaluationData& OutData)
{
	const FDateTime CurrentTime = bUseUtc ? FDateTime::UtcNow() : FDateTime::Now();

	switch(Mode)
	{
	case EPropertyAnimatorCoreSystemMode::LocalTime:
		{
			OutData.TimeElapsed = (CurrentTime - FDateTime::MinValue()).GetTotalSeconds();
		}
		break;
	case EPropertyAnimatorCoreSystemMode::Countdown:
		{
			if (CountdownFormat == EPropertyAnimatorCoreSystemCountdownFormat::Target)
			{
				const FDateTime TargetTime = CurrentTime.GetDate() + CountdownTimeSpan;
				OutData.TimeElapsed = (TargetTime - CurrentTime).GetTotalSeconds();
			}
			else
			{
				OutData.TimeElapsed = (CountdownTimeSpan - (CurrentTime - ActivationTime)).GetTotalSeconds();
			}
		}
		break;
	case EPropertyAnimatorCoreSystemMode::Stopwatch:
		{
			OutData.TimeElapsed = (CurrentTime - ActivationTime).GetTotalSeconds();
		}
		break;
	}

	return true;
}

void UPropertyAnimatorCoreSystemTimeSource::OnTimeSourceActive()
{
	Super::OnTimeSourceActive();

	SetActivationTime();
	OnModeChanged();
}

bool UPropertyAnimatorCoreSystemTimeSource::ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue)
{
	if (Super::ImportPreset(InPreset, InValue) && InValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> ObjectArchive = InValue->AsMutableObject();

		bool bUseUtcValue = bUseUtc;
		ObjectArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreSystemTimeSource, bUseUtc), bUseUtcValue);
		SetUseUtc(bUseUtcValue);
		
		uint64 CountdownFormatValue = static_cast<uint64>(CountdownFormat);
		ObjectArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreSystemTimeSource, CountdownFormat), CountdownFormatValue);
		SetCountdownFormat(static_cast<EPropertyAnimatorCoreSystemCountdownFormat>(CountdownFormatValue));

		FString CountdownValue = CountdownDuration;
		ObjectArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreSystemTimeSource, CountdownDuration), CountdownValue);
		SetCountdownDuration(CountdownValue);

		uint64 ModeValue = static_cast<uint64>(Mode);
		ObjectArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreSystemTimeSource, Mode), ModeValue);
		SetMode(static_cast<EPropertyAnimatorCoreSystemMode>(ModeValue));

		return true;
	}

	return false;
}

bool UPropertyAnimatorCoreSystemTimeSource::ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const
{
	if (Super::ExportPreset(InPreset, OutValue) && OutValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> ObjectArchive = OutValue->AsMutableObject();

		ObjectArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreSystemTimeSource, Mode), static_cast<uint64>(Mode));
		ObjectArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreSystemTimeSource, bUseUtc), bUseUtc);
		ObjectArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreSystemTimeSource, CountdownFormat), static_cast<uint64>(CountdownFormat));
		ObjectArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreSystemTimeSource, CountdownDuration), CountdownDuration);

		return true;
	}

	return false;
}

void UPropertyAnimatorCoreSystemTimeSource::SetMode(EPropertyAnimatorCoreSystemMode InMode)
{
	if (Mode == InMode)
	{
		return;
	}

	Mode = InMode;
	OnModeChanged();
}

void UPropertyAnimatorCoreSystemTimeSource::SetUseUtc(bool bInUseUtc)
{
	if (bUseUtc == bInUseUtc)
	{
		return;
	}

	bUseUtc = bInUseUtc;
	SetActivationTime();
}

void UPropertyAnimatorCoreSystemTimeSource::SetCountdownFormat(EPropertyAnimatorCoreSystemCountdownFormat InFormat)
{
	CountdownFormat = InFormat;
}

void UPropertyAnimatorCoreSystemTimeSource::SetCountdownDuration(const FTimespan& InTimeSpan)
{
	if (InTimeSpan == CountdownTimeSpan)
	{
		return;
	}

	SetCountdownDuration(InTimeSpan.ToString(TEXT("%h:%m:%s")));
}

void UPropertyAnimatorCoreSystemTimeSource::SetCountdownDuration(const FString& InDuration)
{
	if (CountdownDuration == InDuration)
	{
		return;
	}

	CountdownDuration = InDuration;
	OnModeChanged();
}

#if WITH_EDITOR
void UPropertyAnimatorCoreSystemTimeSource::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreSystemTimeSource, Mode)
		|| MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreSystemTimeSource, CountdownDuration))
	{
		OnModeChanged();
	}
	else if (MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreSystemTimeSource, bUseUtc))
	{
		SetActivationTime();
	}
}
#endif

void UPropertyAnimatorCoreSystemTimeSource::OnModeChanged()
{
	if (Mode == EPropertyAnimatorCoreSystemMode::Countdown)
	{
		CountdownTimeSpan = ParseTime(CountdownDuration);
	}
}

void UPropertyAnimatorCoreSystemTimeSource::SetActivationTime()
{
	ActivationTime = bUseUtc ? FDateTime::UtcNow() : FDateTime::Now();
}

FTimespan UPropertyAnimatorCoreSystemTimeSource::ParseTime(const FString& InFormat)
{
	// Regex patterns for different formats
	static const FRegexPattern HHMMSSPattern(TEXT("^(?:(\\d{2}):)?(\\d{2}):(\\d{2})$")); // 01:00 00:01:00
	static const FRegexPattern CombinedPattern(TEXT("(?:(\\d+)h)? ?(?:(\\d+)m)? ?(?:(\\d+)s)?")); // 1h 1m 1s

	FRegexMatcher HHMMSSMatcher(HHMMSSPattern, InFormat);
	FRegexMatcher CombinedMatcher(CombinedPattern, InFormat);

	FTimespan ParsedTimeSpan = FTimespan::Zero();

	if (InFormat.IsNumeric())
	{
		const int32 Seconds = FCString::Atoi(*InFormat);
		ParsedTimeSpan = FTimespan::FromSeconds(Seconds);
	}
	else if (HHMMSSMatcher.FindNext())
	{
		const int32 Hours = HHMMSSMatcher.GetCaptureGroup(1).IsEmpty() ? 0 : FCString::Atoi(*HHMMSSMatcher.GetCaptureGroup(1));
		const int32 Minutes = FCString::Atoi(*HHMMSSMatcher.GetCaptureGroup(2));
		const int32 Seconds = FCString::Atoi(*HHMMSSMatcher.GetCaptureGroup(3));
		ParsedTimeSpan = FTimespan::FromHours(Hours) + FTimespan::FromMinutes(Minutes) + FTimespan::FromSeconds(Seconds);
	}
	else if (CombinedMatcher.FindNext())
	{
		const int32 Hours = CombinedMatcher.GetCaptureGroup(1).IsEmpty() ? 0 : FCString::Atoi(*CombinedMatcher.GetCaptureGroup(1));
		const int32 Minutes = CombinedMatcher.GetCaptureGroup(2).IsEmpty() ? 0 : FCString::Atoi(*CombinedMatcher.GetCaptureGroup(2));
		const int32 Seconds = CombinedMatcher.GetCaptureGroup(3).IsEmpty() ? 0 : FCString::Atoi(*CombinedMatcher.GetCaptureGroup(3));
		ParsedTimeSpan = FTimespan::FromHours(Hours) + FTimespan::FromMinutes(Minutes) + FTimespan::FromSeconds(Seconds);
	}

	return ParsedTimeSpan;
}
