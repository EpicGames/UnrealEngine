// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimeSources/PropertyAnimatorCoreTimeSourceBase.h"

#include "Presets/PropertyAnimatorCorePresetArchive.h"
#include "Presets/PropertyAnimatorCorePresetBase.h"

void UPropertyAnimatorCoreTimeSourceBase::ActivateTimeSource()
{
	if (IsTimeSourceActive())
	{
		return;
	}

	bTimeSourceActive = true;
	OnTimeSourceActive();
}

void UPropertyAnimatorCoreTimeSourceBase::DeactivateTimeSource()
{
	if (!IsTimeSourceActive())
	{
		return;
	}

	bTimeSourceActive = false;
	OnTimeSourceInactive();
}

EPropertyAnimatorCoreTimeSourceResult UPropertyAnimatorCoreTimeSourceBase::FetchEvaluationData(FPropertyAnimatorCoreTimeSourceEvaluationData& OutEvaluationData)
{
	if (!UpdateEvaluationData(OutEvaluationData))
	{
		// Reset evaluation state
		return EPropertyAnimatorCoreTimeSourceResult::Idle;
	}

	if (!IsFramerateAllowed(OutEvaluationData.TimeElapsed))
	{
		// Skip evaluation for this run
		return EPropertyAnimatorCoreTimeSourceResult::Skip;
	}

	LastTimeElapsed = OutEvaluationData.TimeElapsed;

	return EPropertyAnimatorCoreTimeSourceResult::Evaluate;
}

void UPropertyAnimatorCoreTimeSourceBase::SetFrameRate(float InFrameRate)
{
	FrameRate = FMath::Max(UE_KINDA_SMALL_NUMBER, InFrameRate);
}

void UPropertyAnimatorCoreTimeSourceBase::SetUseFrameRate(bool bInUseFrameRate)
{
	bUseFrameRate = bInUseFrameRate;
}

bool UPropertyAnimatorCoreTimeSourceBase::ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue)
{
	TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> ObjectArchive = InValue->AsMutableObject();

	if (!ObjectArchive)
	{
		return false;
	}

	bool bUseFrameRateValue = bUseFrameRate;
	ObjectArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreTimeSourceBase, bUseFrameRate), bUseFrameRateValue);
	SetUseFrameRate(bUseFrameRateValue);

	double FrameRateValue = FrameRate;
	ObjectArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreTimeSourceBase, FrameRate), FrameRateValue);
	SetFrameRate(FrameRateValue);

	return true;
}

bool UPropertyAnimatorCoreTimeSourceBase::ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const
{
	const TSharedRef<FPropertyAnimatorCorePresetObjectArchive> ObjectArchive = InPreset->GetArchiveImplementation()->CreateObject();
	OutValue = ObjectArchive;

	ObjectArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreTimeSourceBase, bUseFrameRate), bUseFrameRate);
	ObjectArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCoreTimeSourceBase, FrameRate), bUseFrameRate);

	return true;
}

bool UPropertyAnimatorCoreTimeSourceBase::UpdateEvaluationData(FPropertyAnimatorCoreTimeSourceEvaluationData& OutData)
{
	return false;
}

bool UPropertyAnimatorCoreTimeSourceBase::IsFramerateAllowed(double InNewTime) const
{
	return !bUseFrameRate || FMath::IsNearlyZero(FrameRate) || FMath::Abs(InNewTime - LastTimeElapsed) > FMath::Abs(1.f / FrameRate);
}
