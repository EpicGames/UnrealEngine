// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimeSources/PropertyAnimatorCoreWorldTimeSource.h"

#include "Engine/World.h"

bool UPropertyAnimatorCoreWorldTimeSource::UpdateEvaluationData(FPropertyAnimatorCoreTimeSourceEvaluationData& OutData)
{
	const UWorld* World = GetWorld();

	if (!World)
	{
		return false;
	}

	OutData.TimeElapsed = World->GetTimeSeconds();

	return World->IsEditorWorld()
		|| World->IsPreviewWorld()
		|| (World->IsGameWorld() && World->HasBegunPlay());
}
