// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PropertyAnimatorCore : ModuleRules
{
	public PropertyAnimatorCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"MovieScene"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"Json"
			}
		);

		if (Target.Type == TargetRules.TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
				{
					"UnrealEd"
				}
			);
		}

		ShortName = "PropAnimCore";
	}
}
