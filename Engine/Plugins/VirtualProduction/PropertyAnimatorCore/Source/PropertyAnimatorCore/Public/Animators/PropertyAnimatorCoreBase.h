// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Presets/PropertyAnimatorCorePresetable.h"
#include "Properties/PropertyAnimatorCoreContext.h"
#include "Properties/PropertyAnimatorCoreData.h"
#include "UObject/Object.h"
#include "PropertyAnimatorCoreBase.generated.h"

class FObjectPreSaveContext;
class UPropertyAnimatorCoreComponent;
class UPropertyAnimatorCoreTimeSourceBase;

UENUM(BlueprintType)
enum class EPropertyAnimatorPropertySupport : uint8
{
	None = 0,
	Incomplete = 1 << 0,
	Complete = 1 << 1,
	All = Incomplete | Complete
};

struct FPropertyAnimatorCoreMetadata
{
	FName Name = NAME_None;
	FText DisplayName = FText::GetEmpty();
	FText Description = FText::GetEmpty();
	FName Category = TEXT("Default");
};

enum class EPropertyAnimatorCoreUpdateEvent : uint8
{
	User,
	Undo,
	Load,
	Duplicate,
	Destroyed
};

/** Abstract base class for any Animator, holds a set of linked properties */
UCLASS(MinimalAPI, Abstract, EditInlineNew, AutoExpandCategories=("Animator"))
class UPropertyAnimatorCoreBase : public UObject, public IPropertyAnimatorCorePresetable
{
	GENERATED_BODY()

	friend class UPropertyAnimatorCoreComponent;
	friend class UPropertyAnimatorCoreContext;

public:
	static constexpr const TCHAR* TimeElapsedParameterName = TEXT("TimeElapsed");
	static constexpr const TCHAR* MagnitudeParameterName = TEXT("Magnitude");
	static constexpr const TCHAR* FrequencyParameterName = TEXT("Frequency");
	static constexpr const TCHAR* AlphaParameterName = TEXT("Alpha");

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnAnimatorUpdated, UPropertyAnimatorCoreComponent* /* InComponent */, UPropertyAnimatorCoreBase* /* InAnimator */, EPropertyAnimatorCoreUpdateEvent /** InType */)
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAnimatorPropertyUpdated, UPropertyAnimatorCoreBase* /* InAnimator */, const FPropertyAnimatorCoreData& /* InProperty */)

	static FOnAnimatorUpdated::RegistrationType& OnPropertyAnimatorAdded()
	{
		return OnAnimatorAddedDelegate;
	}

	static FOnAnimatorUpdated::RegistrationType& OnPropertyAnimatorRemoved()
	{
		return OnAnimatorRemovedDelegate;
	}

	static FOnAnimatorUpdated::RegistrationType& OnPropertyAnimatorRenamed()
	{
		return OnAnimatorRenamedDelegate;
	}

	static FOnAnimatorPropertyUpdated::RegistrationType& OnPropertyAnimatorPropertyLinked()
	{
		return OnAnimatorPropertyLinkedDelegate;
	}

	static FOnAnimatorPropertyUpdated::RegistrationType& OnPropertyAnimatorPropertyUnlinked()
	{
		return OnAnimatorPropertyUnlinkedDelegate;
	}

#if WITH_EDITOR
	PROPERTYANIMATORCORE_API static FName GetAnimatorEnabledPropertyName();
	PROPERTYANIMATORCORE_API static FName GetLinkedPropertiesPropertyName();
#endif
	
	PROPERTYANIMATORCORE_API UPropertyAnimatorCoreBase();

	PROPERTYANIMATORCORE_API AActor* GetAnimatorActor() const;

	PROPERTYANIMATORCORE_API UPropertyAnimatorCoreComponent* GetAnimatorComponent() const;

	/** Set the state of this animator */
	PROPERTYANIMATORCORE_API void SetAnimatorEnabled(bool bInIsEnabled);
	bool GetAnimatorEnabled() const
	{
		return bAnimatorEnabled;
	}

	PROPERTYANIMATORCORE_API void SetOverrideTimeSource(bool bInOverride);
	bool GetOverrideTimeSource() const
	{
		return bOverrideTimeSource;
	}

	/** Set the time source name to use */
	PROPERTYANIMATORCORE_API void SetTimeSourceName(FName InTimeSourceName);
	FName GetTimeSourceName() const
	{
		return TimeSourceName;
	}

	/** Get the active time source */
	PROPERTYANIMATORCORE_API UPropertyAnimatorCoreTimeSourceBase* GetActiveTimeSource() const;

	/** Set the display name of this animator */
	PROPERTYANIMATORCORE_API void SetAnimatorDisplayName(FName InName);

	FName GetAnimatorDisplayName() const
	{
		return AnimatorDisplayName;
	}

	/** Gets the animator metadata */
	PROPERTYANIMATORCORE_API TSharedRef<const FPropertyAnimatorCoreMetadata> GetAnimatorMetadata() const;

	/** Get all linked properties within this animator */
	PROPERTYANIMATORCORE_API TSet<FPropertyAnimatorCoreData> GetLinkedProperties() const;

	/** Get linked properties count within this animator */
	PROPERTYANIMATORCORE_API int32 GetLinkedPropertiesCount() const;

	/** Link property to this Animator to be able to drive it */
	PROPERTYANIMATORCORE_API UPropertyAnimatorCoreContext* LinkProperty(const FPropertyAnimatorCoreData& InLinkProperty);

	/** Unlink property from this Animator */
	PROPERTYANIMATORCORE_API bool UnlinkProperty(const FPropertyAnimatorCoreData& InUnlinkProperty);

	/** Checks if this Animator is controlling this property */
	PROPERTYANIMATORCORE_API bool IsPropertyLinked(const FPropertyAnimatorCoreData& InPropertyData) const;

	/** Checks if this animator is controlling all properties */
	bool IsPropertiesLinked(const TSet<FPropertyAnimatorCoreData>& InProperties) const;

	/** Returns all inner properties that are controlled by this Animator linked to member property */
	PROPERTYANIMATORCORE_API TSet<FPropertyAnimatorCoreData> GetInnerPropertiesLinked(const FPropertyAnimatorCoreData& InPropertyData) const;

	/**
	 * Checks recursively for properties that are supported by this Animator, calls IsPropertySupported to check
	 * Stops when the InSearchDepth has been reached otherwise continues to gather supported properties
	 */
	PROPERTYANIMATORCORE_API bool GetPropertiesSupported(const FPropertyAnimatorCoreData& InPropertyData, TSet<FPropertyAnimatorCoreData>& OutProperties, uint8 InSearchDepth = 1, EPropertyAnimatorPropertySupport InSupportExpected = EPropertyAnimatorPropertySupport::All) const;

	/** Retrieves the support level of a property */
	PROPERTYANIMATORCORE_API EPropertyAnimatorPropertySupport GetPropertySupport(const FPropertyAnimatorCoreData& InPropertyData) const;

	/** Checks if a property support is available */
	PROPERTYANIMATORCORE_API bool HasPropertySupport(const FPropertyAnimatorCoreData& InPropertyData, EPropertyAnimatorPropertySupport InSupportExpected = EPropertyAnimatorPropertySupport::All) const;

	/** Override this to check if a property is supported by this animator */
	virtual EPropertyAnimatorPropertySupport IsPropertySupported(const FPropertyAnimatorCoreData& InPropertyData) const
	{
		return EPropertyAnimatorPropertySupport::None;
	}

	/** Checks whether a time source is supported on this animator */
	virtual bool IsTimeSourceSupported(UPropertyAnimatorCoreTimeSourceBase* InTimeSource) const
	{
		return true;
	}

	/** Get the context for the linked property */
	PROPERTYANIMATORCORE_API UPropertyAnimatorCoreContext* GetLinkedPropertyContext(const FPropertyAnimatorCoreData& InProperty) const;

	TConstArrayView<UPropertyAnimatorCoreContext*> GetLinkedPropertiesContext() const
	{
		return LinkedProperties;
	}

	/** Get the casted context for the linked property */
	template<typename InContextClass
		UE_REQUIRES(TIsDerivedFrom<InContextClass, UPropertyAnimatorCoreContext>::Value)>
	InContextClass* GetLinkedPropertyContext(const FPropertyAnimatorCoreData& InProperty) const
	{
		return Cast<InContextClass>(GetLinkedPropertyContext(InProperty));
	}

	/** Use this to process each linked properties and resolve it, even virtual ones */
	template<typename InContextClass
		UE_REQUIRES(TIsDerivedFrom<InContextClass, UPropertyAnimatorCoreContext>::Value)>
	bool ForEachLinkedProperty(TFunctionRef<bool(InContextClass*, const FPropertyAnimatorCoreData&)> InFunction, bool bInResolve = true) const
	{
		for (const TObjectPtr<UPropertyAnimatorCoreContext>& LinkedProperty : LinkedProperties)
		{
			if (InContextClass* PropertyContext = Cast<InContextClass>(LinkedProperty.Get()))
			{
				if (bInResolve)
				{
					for (const FPropertyAnimatorCoreData& ResolvedPropertyData : PropertyContext->ResolveProperty(false))
					{
						if (!ResolvedPropertyData.IsResolved())
						{
							continue;
						}

						if (!InFunction(PropertyContext, ResolvedPropertyData))
						{
							return false;
						}
					}
				}
				else
				{
					if (!InFunction(PropertyContext, LinkedProperty->GetAnimatedProperty()))
					{
						return false;
					}
				}
			}
		}

		return true;
	}

	//~ Begin IPropertyAnimatorCorePresetable
	PROPERTYANIMATORCORE_API virtual bool ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue) override;
	PROPERTYANIMATORCORE_API virtual bool ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const override;
	//~ End IPropertyAnimatorCorePresetable

protected:
	//~ Begin UObject
	PROPERTYANIMATORCORE_API virtual void PostCDOContruct() override;
	PROPERTYANIMATORCORE_API virtual void BeginDestroy() override;
	PROPERTYANIMATORCORE_API virtual void PostLoad() override;
	PROPERTYANIMATORCORE_API virtual void PostEditImport() override;
	PROPERTYANIMATORCORE_API virtual void PreDuplicate(FObjectDuplicationParameters& InDupParams) override;
	PROPERTYANIMATORCORE_API virtual void PostDuplicate(EDuplicateMode::Type InDuplicateMode) override;
#if WITH_EDITOR
	PROPERTYANIMATORCORE_API virtual void PreEditUndo() override;
	PROPERTYANIMATORCORE_API virtual void PostEditUndo() override;
	PROPERTYANIMATORCORE_API virtual void PreEditChange(FProperty* InPropertyAboutToChange) override;
	PROPERTYANIMATORCORE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	/** Update display name based on linked properties */
	void UpdateAnimatorDisplayName();

	/** Used to evaluate linked properties, assign the result in the property bag and return true on success to update property value */
	void EvaluateEachLinkedProperty(
		TFunctionRef<bool(
			UPropertyAnimatorCoreContext* /** InPropertyContext */
			, const FPropertyAnimatorCoreData& /** InResolvedProperty */
			, FInstancedPropertyBag& /** OutEvaluation */
			, int32 InRangeIndex
			, int32 InRangeMax)> InFunction
		)
	{
		checkf(bEvaluatingProperties, TEXT("EvaluateEachLinkedProperty can only be called in EvaluateProperties"))

		for (const TObjectPtr<UPropertyAnimatorCoreContext>& LinkedProperty : LinkedProperties)
		{
			UPropertyAnimatorCoreContext* PropertyContext = LinkedProperty.Get();

			if (!PropertyContext || !PropertyContext->IsAnimated())
			{
				continue;
			}

			const TArray<FPropertyAnimatorCoreData> ResolvedProperties = PropertyContext->ResolveProperty(/** ForEvaluation */true);

			for (int32 Index = 0; Index < ResolvedProperties.Num(); Index++)
			{
				const FPropertyAnimatorCoreData& ResolvedPropertyData = ResolvedProperties[Index];

				if (!ResolvedPropertyData.IsResolved())
				{
					continue;
				}

				if (InFunction(PropertyContext, ResolvedPropertyData, EvaluatedPropertyValues, Index, ResolvedProperties.Num() - 1))
				{
					PropertyContext->CommitEvaluationResult(ResolvedPropertyData, EvaluatedPropertyValues);
				}
			}
		}
	}

	virtual void OnAnimatorDisplayNameChanged() {}

	PROPERTYANIMATORCORE_API virtual void OnAnimatorAdded(EPropertyAnimatorCoreUpdateEvent InType);
	PROPERTYANIMATORCORE_API virtual void OnAnimatorRemoved(EPropertyAnimatorCoreUpdateEvent InType);

	PROPERTYANIMATORCORE_API virtual void OnAnimatorEnabled(EPropertyAnimatorCoreUpdateEvent InType);
	PROPERTYANIMATORCORE_API virtual void OnAnimatorDisabled(EPropertyAnimatorCoreUpdateEvent InType);

	virtual void OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata) {}

	virtual void OnTimeSourceChanged() {}

	/** Returns the property context class to use */
	PROPERTYANIMATORCORE_API virtual TSubclassOf<UPropertyAnimatorCoreContext> GetPropertyContextClass(const FPropertyAnimatorCoreData& InProperty);

	virtual void OnPropertyLinked(UPropertyAnimatorCoreContext* InLinkedProperty, EPropertyAnimatorPropertySupport InSupport) {}
	virtual void OnPropertyUnlinked(UPropertyAnimatorCoreContext* InUnlinkedProperty) {}

	/** Apply animators effect on linked properties */
	virtual void EvaluateProperties(FInstancedPropertyBag& InParameters) {}

private:
	/** Called when a Animator is created */
	PROPERTYANIMATORCORE_API static FOnAnimatorUpdated OnAnimatorAddedDelegate;

	/** Called when a Animator is removed */
	PROPERTYANIMATORCORE_API static FOnAnimatorUpdated OnAnimatorRemovedDelegate;

	/** Called when a Animator is renamed */
	PROPERTYANIMATORCORE_API static FOnAnimatorUpdated OnAnimatorRenamedDelegate;

	/** Called when a property is linked to a Animator */
	PROPERTYANIMATORCORE_API static FOnAnimatorPropertyUpdated OnAnimatorPropertyLinkedDelegate;

	/** Called when a property is unlinked to a Animator */
	PROPERTYANIMATORCORE_API static FOnAnimatorPropertyUpdated OnAnimatorPropertyUnlinkedDelegate;

	/** Restore modified properties to original state */
	void RestoreProperties();

	/** Allocate and saves properties in the property bag */
	void SaveProperties();

	/** Called by the component to evaluate this animator */
	void EvaluateAnimator(FInstancedPropertyBag& InParameters);

	void OnObjectReplaced(const TMap<UObject*, UObject*>& InReplacementMap);

#if WITH_EDITOR
	/** Needed to restore properties and stop animation before world is saved */
	void OnPreSaveWorld(UWorld* InWorld, FObjectPreSaveContext InContext);

	/** Needed to restore properties and reset animation before PIE is started */
	void OnPreBeginPIE(bool bInSimulating);
#endif

	void OnAnimatorEnabledChanged(EPropertyAnimatorCoreUpdateEvent InType);

	void CleanLinkedProperties();

	void OnTimeSourceNameChanged();

	/** Called when time source enters idle/invalid state */
	void OnTimeSourceEnterIdleState();

	/** Called after an action that causes the owner to change */
	void ResolvePropertiesOwner(AActor* InNewOwner = nullptr);

	UPropertyAnimatorCoreTimeSourceBase* FindOrAddTimeSource(FName InTimeSourceName);

	UFUNCTION()
	TArray<FName> GetTimeSourceNames() const;

	/** Enable control of properties linked to this Animator */
	UPROPERTY(EditInstanceOnly, Getter="GetAnimatorEnabled", Setter="SetAnimatorEnabled", Category="Animator", meta=(DisplayPriority="0", AllowPrivateAccess="true"))
	bool bAnimatorEnabled = true;

	/** Display name as title property for component array, hide it but must be visible to editor for array title property */
	UPROPERTY(VisibleInstanceOnly, Category="Animator", meta=(EditCondition="false", EditConditionHides))
	FName AnimatorDisplayName;

	/** Context for properties linked to this Animator */
	UPROPERTY(EditInstanceOnly, NoClear, Export, Instanced, EditFixedSize, Category="Animator", meta=(EditFixedOrder))
	TArray<TObjectPtr<UPropertyAnimatorCoreContext>> LinkedProperties;

	/** Use the global time source or override it on this animator */
	UPROPERTY(EditInstanceOnly, Setter="SetOverrideTimeSource", Getter="GetOverrideTimeSource", Category="Animator", meta=(InlineEditConditionToggle))
	bool bOverrideTimeSource = true;

	/** The time source to use */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(GetOptions="GetTimeSourceNames", EditCondition="bOverrideTimeSource", NoResetToDefault))
	FName TimeSourceName = NAME_None;

	/** Active time source with its options, determined by its name */
	UPROPERTY(VisibleInstanceOnly, Instanced, Transient, DuplicateTransient, Category="Animator", meta=(EditCondition="bOverrideTimeSource", EditConditionHides, HideEditConditionToggle))
	TObjectPtr<UPropertyAnimatorCoreTimeSourceBase> ActiveTimeSource;

	/** The cached time source used by this Animator */
	UE_DEPRECATED(5.5, "Use TimeSources instead")
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use TimeSources instead"))
	TMap<FName, TObjectPtr<UPropertyAnimatorCoreTimeSourceBase>> TimeSourcesInstances;

	/** Cached time sources used by this animator */
	UPROPERTY()
	TArray<TObjectPtr<UPropertyAnimatorCoreTimeSourceBase>> TimeSources;

	/** Evaluated property container, reset on every update round */
	UPROPERTY(Transient, DuplicateTransient, NonTransactional)
	FInstancedPropertyBag EvaluatedPropertyValues;

	/** Are we evaluating properties currently */
	bool bEvaluatingProperties = false;

	/** Animator metadata, same for all instances of this class */
	TSharedPtr<FPropertyAnimatorCoreMetadata> Metadata;
};