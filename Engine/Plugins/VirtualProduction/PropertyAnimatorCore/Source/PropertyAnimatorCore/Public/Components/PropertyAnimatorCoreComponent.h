// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "Animators/PropertyAnimatorCoreBase.h"
#include "PropertyAnimatorCoreComponent.generated.h"

/** A container for controllers that holds properties in this actor */
UCLASS(MinimalAPI, ClassGroup=(Custom), AutoExpandCategories=("Animator"), HideCategories=("Activation", "Cooking", "AssetUserData", "Collision", "Tags", "ComponentReplication", "Navigation", "Variable", "Replication"), meta=(BlueprintSpawnableComponent))
class UPropertyAnimatorCoreComponent : public UActorComponent
{
	GENERATED_BODY()

	friend class UPropertyAnimatorCoreSubsystem;

public:
	/** Create an instance of this component class and adds it to an actor */
	PROPERTYANIMATORCORE_API static UPropertyAnimatorCoreComponent* FindOrAdd(AActor* InActor);

#if WITH_EDITOR
	PROPERTYANIMATORCORE_API static FName GetAnimatorsEnabledPropertyName();
	PROPERTYANIMATORCORE_API static FName GetPropertyAnimatorsPropertyName();
#endif

	UPropertyAnimatorCoreComponent();

	PROPERTYANIMATORCORE_API void SetAnimators(const TArray<TObjectPtr<UPropertyAnimatorCoreBase>>& InAnimators);
	TConstArrayView<TObjectPtr<UPropertyAnimatorCoreBase>> GetAnimators() const
	{
		return PropertyAnimators;
	}

	int32 GetAnimatorsCount() const
	{
		return PropertyAnimators.Num();
	}

	/** Set the state of all animators in this component */
	void SetAnimatorsEnabled(bool bInEnabled);
	bool GetAnimatorsEnabled() const
	{
		return bAnimatorsEnabled;
	}

	/** Set the magnitude for all animators in this component */
	void SetAnimatorsMagnitude(float InMagnitude);
	float GetAnimatorsMagnitude() const
	{
		return AnimatorsMagnitude;
	}

	void SetAnimatorsTimeSourceName(FName InTimeSourceName);
	FName GetAnimatorsTimeSourceName() const
	{
		return AnimatorsTimeSourceName;
	}

	UPropertyAnimatorCoreTimeSourceBase* GetAnimatorsActiveTimeSource() const
	{
		return ActiveAnimatorsTimeSource;
	}

	/** Process a function for each controller, stops when false is returned otherwise continue until the end */
	PROPERTYANIMATORCORE_API void ForEachAnimator(TFunctionRef<bool(UPropertyAnimatorCoreBase*)> InFunction) const;

	/** Checks if this component animators should be active */
	bool ShouldAnimate() const;

protected:
	static FName GetAnimatorName(const UPropertyAnimatorCoreBase* InAnimator);

	//~ Begin UActorComponent
	virtual void OnComponentCreated() override;
	virtual void OnComponentDestroyed(bool bInDestroyingHierarchy) override;
	virtual void TickComponent(float InDeltaTime, ELevelTick InTickType, FActorComponentTickFunction* InTickFunction) override;
	//~ End UActorComponent

	//~ Begin UObject
	virtual void PostLoad() override;
	virtual void PostEditImport() override;
	virtual void PostDuplicate(EDuplicateMode::Type InMode) override;
#if WITH_EDITOR
	virtual void PreEditUndo() override;
	virtual void PostEditUndo() override;
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	/** Adds a new controller and returns it casted */
	template<typename InAnimatorClass = UPropertyAnimatorCoreBase, typename = typename TEnableIf<TIsDerivedFrom<InAnimatorClass, UPropertyAnimatorCoreBase>::Value>::Type>
	InAnimatorClass* AddAnimator()
	{
		const UClass* AnimatorClass = InAnimatorClass::StaticClass();
		return Cast<InAnimatorClass>(AddAnimator(AnimatorClass));
	}

	/** Adds a new animator of that class */
	UPropertyAnimatorCoreBase* AddAnimator(const UClass* InAnimatorClass);

	/** Clones an existing animator */
	UPropertyAnimatorCoreBase* CloneAnimator(UPropertyAnimatorCoreBase* InAnimator);

	/** Removes an existing animator */
	bool RemoveAnimator(UPropertyAnimatorCoreBase* InAnimator);

	/** Change global state for animators */
	void OnAnimatorsSetEnabled(const UWorld* InWorld, bool bInEnabled, bool bInTransact);

	/** Callback when PropertyAnimators changed */
	void OnAnimatorsChanged(EPropertyAnimatorCoreUpdateEvent InType);

	/** Callback when global enabled state is changed */
	void OnAnimatorsEnabledChanged(EPropertyAnimatorCoreUpdateEvent InType);

	/** Callback when global time source name is changed */
	void OnTimeSourceNameChanged();

	/** Evaluate only specified animators */
	bool EvaluateAnimators();

	/** Finds a cached time source with this name or creates a new one */
	UPropertyAnimatorCoreTimeSourceBase* FindOrAddTimeSource(FName InTimeSourceName);

	UFUNCTION()
	TArray<FName> GetTimeSourceNames() const;

	/** Animators linked to this actor, they contain only properties within this actor */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, NoClear, Export, Instanced, Setter="SetAnimators", Category="Animator", meta=(TitleProperty="AnimatorDisplayName"))
	TArray<TObjectPtr<UPropertyAnimatorCoreBase>> PropertyAnimators;

	/** Global state for all animators controlled by this component */
	UPROPERTY(EditInstanceOnly, Getter="GetAnimatorsEnabled", Setter="SetAnimatorsEnabled", Category="Animator", meta=(DisplayPriority="0", AllowPrivateAccess="true"))
	bool bAnimatorsEnabled = true;

	/** Global magnitude for all animators controlled by this component */
	UPROPERTY(EditInstanceOnly, Getter, Setter, DisplayName="Global Magnitude", Category="Animator", meta=(ClampMin="0", ClampMax="1", UIMin="0", UIMax="1", AllowPrivateAccess="true"))
	float AnimatorsMagnitude = 1.f;

	/** The global time source to use, can be overriden in animator */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Global Time Source Name", Category="Animator", meta=(GetOptions="GetTimeSourceNames", NoResetToDefault))
	FName AnimatorsTimeSourceName = NAME_None;

	/** Active time source with its options, determined by its name */
	UPROPERTY(VisibleInstanceOnly, Instanced, Transient, DuplicateTransient, Category="Animator")
	TObjectPtr<UPropertyAnimatorCoreTimeSourceBase> ActiveAnimatorsTimeSource;

private:
	/** Deprecated property set, will be migrated to PropertyAnimators property on load */
	UE_DEPRECATED(5.5, "Moved to PropertyAnimators")
	UPROPERTY()
	TSet<TObjectPtr<UPropertyAnimatorCoreBase>> Animators;

	/** Transient copy of property animators when changes are detected to see the diff only */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient, NonTransactional)
	TArray<TObjectPtr<UPropertyAnimatorCoreBase>> PropertyAnimatorsInternal;

	/** Cached time sources used by this animator component */
	UPROPERTY()
	TArray<TObjectPtr<UPropertyAnimatorCoreTimeSourceBase>> TimeSources;
};