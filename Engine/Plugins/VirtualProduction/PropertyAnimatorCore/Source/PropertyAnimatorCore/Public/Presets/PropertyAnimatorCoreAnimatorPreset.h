// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyAnimatorCorePresetBase.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "PropertyAnimatorCoreAnimatorPreset.generated.h"

class FJsonValue;

/**
 * Animator preset class used to import/export animator data
 */
UCLASS(MinimalAPI, BlueprintType)
class UPropertyAnimatorCoreAnimatorPreset : public UPropertyAnimatorCorePresetBase
{
	GENERATED_BODY()

public:
	//~ Begin UPropertyAnimatorCorePresetBase
	virtual bool IsPresetApplied(const UPropertyAnimatorCoreBase* InAnimator) const override;
	virtual bool IsPresetSupported(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator) const override;
	virtual bool ApplyPreset(UPropertyAnimatorCoreBase* InAnimator) override;
	virtual bool UnapplyPreset(UPropertyAnimatorCoreBase* InAnimator) override;
	virtual void CreatePreset(FName InName, const TArray<IPropertyAnimatorCorePresetable*>& InPresetableItem) override;
	virtual bool LoadPreset() override;
	//~ End UPropertyAnimatorCorePresetBase

	PROPERTYANIMATORCORE_API UPropertyAnimatorCoreBase* GetAnimatorTemplate() const;

protected:
	/** Animator class to target for this preset */
	UPROPERTY(Transient)
	TSubclassOf<UPropertyAnimatorCoreBase> TargetAnimatorClass;

	TSharedPtr<FPropertyAnimatorCorePresetArchive> AnimatorPreset;
};
