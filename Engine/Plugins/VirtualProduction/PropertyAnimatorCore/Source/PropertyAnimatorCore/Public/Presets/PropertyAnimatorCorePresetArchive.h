// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"

struct FPropertyAnimatorCorePresetArchive;
struct FPropertyAnimatorCorePresetObjectArchive;
struct FPropertyAnimatorCorePresetArrayArchive;
struct FPropertyAnimatorCorePresetValueArchive;

/** Enumerates all possible value type stored in archive */
enum class EPropertyAnimatorCorePresetArchiveType : uint8
{
	/** Used for struct, object, maps */
	Object,
	/** Used for array, set */
	Array,
	/** Used for primitive types like number, string, bool */
	Value
};

/** Represents a custom implementation to create value in the implementation format, should be a singleton */
struct FPropertyAnimatorCorePresetArchiveImplementation : TSharedFromThis<FPropertyAnimatorCorePresetArchiveImplementation>
{
	virtual ~FPropertyAnimatorCorePresetArchiveImplementation() = default;

	virtual TSharedRef<FPropertyAnimatorCorePresetObjectArchive> CreateObject() = 0;
	virtual TSharedRef<FPropertyAnimatorCorePresetArrayArchive> CreateArray() = 0;
	virtual TSharedRef<FPropertyAnimatorCorePresetValueArchive> CreateValue(bool bInValue) = 0;
	virtual TSharedRef<FPropertyAnimatorCorePresetValueArchive> CreateValue(uint64 InValue) = 0;
	virtual TSharedRef<FPropertyAnimatorCorePresetValueArchive> CreateValue(int64 InValue) = 0;
	virtual TSharedRef<FPropertyAnimatorCorePresetValueArchive> CreateValue(double InValue) = 0;
	virtual TSharedRef<FPropertyAnimatorCorePresetValueArchive> CreateValue(const FString& InValue) = 0;
	virtual FName GetImplementationType() const = 0;
};

/** Represents an abstract archive for preset system, underlying implementation can vary */
struct FPropertyAnimatorCorePresetArchive : TSharedFromThis<FPropertyAnimatorCorePresetArchive>
{
	virtual ~FPropertyAnimatorCorePresetArchive() = default;

	PROPERTYANIMATORCORE_API bool IsObject() const;
	bool IsArray() const;
	bool IsValue() const;
	EPropertyAnimatorCorePresetArchiveType GetType() const;

	virtual TSharedPtr<const FPropertyAnimatorCorePresetObjectArchive> AsObject() const { return nullptr; }
	virtual TSharedPtr<const FPropertyAnimatorCorePresetArrayArchive> AsArray() const { return nullptr; }
	virtual TSharedPtr<const FPropertyAnimatorCorePresetValueArchive> AsValue() const { return nullptr; }

	virtual TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> AsMutableObject() { return nullptr; }
	virtual TSharedPtr<FPropertyAnimatorCorePresetArrayArchive> AsMutableArray() { return nullptr; }
	virtual TSharedPtr<FPropertyAnimatorCorePresetValueArchive> AsMutableValue() { return nullptr; }

	virtual bool FromString(const FString& InString) = 0;
	virtual bool ToString(FString& OutString) const = 0;

	virtual TSharedRef<FPropertyAnimatorCorePresetArchiveImplementation> GetImplementation() const = 0;
	FName GetImplementationType() const;
};

/** Represents an abstract object archive for preset system, underlying implementation can vary */
struct FPropertyAnimatorCorePresetObjectArchive : FPropertyAnimatorCorePresetArchive
{
	//~ Begin FPropertyAnimatorCorePresetArchive
	virtual TSharedPtr<const FPropertyAnimatorCorePresetObjectArchive> AsObject() const override;
	virtual TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> AsMutableObject() override;
	//~ End FPropertyAnimatorCorePresetArchive

	bool Has(const FString& InKey, TOptional<EPropertyAnimatorCorePresetArchiveType> InType = {}) const;
	virtual bool Remove(const FString& InKey) = 0;
	virtual void Clear() = 0;

	virtual bool Set(const FString& InKey, TSharedRef<FPropertyAnimatorCorePresetArchive> InValue) = 0;
	virtual bool Set(const FString& InKey, bool bInValue) = 0;
	virtual bool Set(const FString& InKey, uint64 InValue) = 0;
	virtual bool Set(const FString& InKey, int64 InValue) = 0;
	virtual bool Set(const FString& InKey, double InValue) = 0;
	virtual bool Set(const FString& InKey, const FString& InValue) = 0;

	virtual bool Get(const FString& InKey, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const = 0;
	virtual bool Get(const FString& InKey, bool& bOutValue) const = 0;
	virtual bool Get(const FString& InKey, uint64& OutValue) const = 0;
	virtual bool Get(const FString& InKey, int64& OutValue) const = 0;
	virtual bool Get(const FString& InKey, double& OutValue) const = 0;
	virtual bool Get(const FString& InKey, FString& OutValue) const = 0;
};

/** Represents an abstract array archive for preset system, underlying implementation can vary */
struct FPropertyAnimatorCorePresetArrayArchive : FPropertyAnimatorCorePresetArchive
{
	//~ Begin FPropertyAnimatorCorePresetArchive
	virtual TSharedPtr<const FPropertyAnimatorCorePresetArrayArchive> AsArray() const override;
	virtual TSharedPtr<FPropertyAnimatorCorePresetArrayArchive> AsMutableArray() override;
	//~ End FPropertyAnimatorCorePresetArchive

	virtual bool Get(int32 InIndex, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const = 0;
	virtual int32 Num() const = 0;
	virtual bool Remove(int32 InIndex) = 0;
	virtual void Clear() = 0;

	virtual bool Add(TSharedRef<FPropertyAnimatorCorePresetArchive> InValue) = 0;
	virtual bool Add(bool bInValue) = 0;
	virtual bool Add(uint64 InValue) = 0;
	virtual bool Add(int64 InValue) = 0;
	virtual bool Add(double InValue) = 0;
	virtual bool Add(const FString& InValue) = 0;
};

/** Represents an abstract value (primitive) archive for preset system, underlying implementation can vary */
struct FPropertyAnimatorCorePresetValueArchive : FPropertyAnimatorCorePresetArchive
{
	//~ Begin FPropertyAnimatorCorePresetArchive
	virtual TSharedPtr<const FPropertyAnimatorCorePresetValueArchive> AsValue() const override;
	virtual TSharedPtr<FPropertyAnimatorCorePresetValueArchive> AsMutableValue() override;
	//~ End FPropertyAnimatorCorePresetArchive

	virtual bool Get(bool& bOutValue) const = 0;
	virtual bool Get(uint64& OutValue) const = 0;
	virtual bool Get(int64& OutValue) const = 0;
	virtual bool Get(double& OutValue) const = 0;
	virtual bool Get(FString& OutValue) const = 0;
};
