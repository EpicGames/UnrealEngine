// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyAnimatorCorePresetArchive.h"
#include "PropertyAnimatorCorePresetBase.generated.h"

class AActor;
class IPropertyAnimatorCorePresetable;
class UPropertyAnimatorCoreBase;

/**
 * Abstract Base class to define preset for animators with custom properties and options
 * Will get registered automatically by the subsystem
 * Should remain transient and stateless
 */
UCLASS(MinimalAPI, BlueprintType, Abstract)
class UPropertyAnimatorCorePresetBase : public UObject
{
	GENERATED_BODY()

public:
	UPropertyAnimatorCorePresetBase()
		: UPropertyAnimatorCorePresetBase(NAME_None)
	{}

	UPropertyAnimatorCorePresetBase(FName InPresetName)
		: PresetName(InPresetName)
	{}

	FName GetPresetName() const
	{
		return PresetName;
	}

	PROPERTYANIMATORCORE_API FText GetPresetDisplayName() const;

	/** Checks if the preset is supported on that actor and animator */
	PROPERTYANIMATORCORE_API virtual bool IsPresetSupported(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator) const PURE_VIRTUAL(UPropertyAnimatorCorePresetBase::IsPresetSupported, return false; );

	/** Checks if the preset is applied to this animator */
	PROPERTYANIMATORCORE_API virtual bool IsPresetApplied(const UPropertyAnimatorCoreBase* InAnimator) const PURE_VIRTUAL(UPropertyAnimatorCorePresetBase::IsPresetApplied, return false; );

	/** Applies this preset on the newly created animator */
	PROPERTYANIMATORCORE_API virtual bool ApplyPreset(UPropertyAnimatorCoreBase* InAnimator) PURE_VIRTUAL(UPropertyAnimatorCorePresetBase::ApplyPreset, return false; );

	/** Un applies this preset on an animator */
	PROPERTYANIMATORCORE_API virtual bool UnapplyPreset(UPropertyAnimatorCoreBase* InAnimator) PURE_VIRTUAL(UPropertyAnimatorCorePresetBase::UnapplyPreset, return false; );

	/** Called once to create this preset out of supported items */
	PROPERTYANIMATORCORE_API virtual void CreatePreset(FName InName, const TArray<IPropertyAnimatorCorePresetable*>& InPresetableItem);

	/** Called once before registering preset to load it, if load fails then preset is not registered */
	virtual bool LoadPreset()
	{
		return true;
	}

	/** Called when this preset is registered by the subsystem */
	virtual void OnPresetRegistered() {}

	/** Called when this preset is unregistered by the subsystem */
	virtual void OnPresetUnregistered() {}

	/** Which implementation to use for archive */
	PROPERTYANIMATORCORE_API virtual TSharedRef<FPropertyAnimatorCorePresetArchiveImplementation> GetArchiveImplementation() const;

protected:
	/** Name used to identify this preset */
	UPROPERTY(VisibleInstanceOnly, Category="Animator")
	FName PresetName;

	/** Name used to display this preset to the user */
	UPROPERTY(VisibleInstanceOnly, Category="Animator")
	FText PresetDisplayName = FText::GetEmpty();

	/** Version of this preset for diffs */
	UPROPERTY(VisibleInstanceOnly, Category="Animator")
	int32 PresetVersion = INDEX_NONE;

	/** Format used for the preset content */
	UPROPERTY(VisibleInstanceOnly, Category="Animator")
	FName PresetFormat;

	/** Preset stringify content */
	UPROPERTY(VisibleInstanceOnly, Category="Animator")
	FString PresetContent;
};