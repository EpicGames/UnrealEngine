// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "Templates/SharedPointer.h"
#include "PropertyAnimatorCorePresetable.generated.h"

struct FPropertyAnimatorCorePresetArchiveImplementation;
class UPropertyAnimatorCorePresetBase;
struct FPropertyAnimatorCorePresetArchive;

UINTERFACE(MinimalAPI)
class UPropertyAnimatorCorePresetable : public UInterface
{
	GENERATED_BODY()
};

/** Preset interface for animator system */
class IPropertyAnimatorCorePresetable
{
	GENERATED_BODY()

public:
	/** Import a specific preset */
	PROPERTYANIMATORCORE_API virtual bool ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue) = 0;

	/** Export a specific preset */
	PROPERTYANIMATORCORE_API virtual bool ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const = 0;
};