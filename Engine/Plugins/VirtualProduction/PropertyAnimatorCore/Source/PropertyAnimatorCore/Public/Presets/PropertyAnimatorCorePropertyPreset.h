// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Properties/PropertyAnimatorCoreData.h"
#include "PropertyAnimatorCorePresetBase.h"
#include "Templates/SharedPointer.h"
#include "PropertyAnimatorCorePropertyPreset.generated.h"

class FJsonValue;

/**
 * Property preset class used to import/export properties on supported animators
 */
UCLASS(MinimalAPI, BlueprintType)
class UPropertyAnimatorCorePropertyPreset : public UPropertyAnimatorCorePresetBase
{
	GENERATED_BODY()

public:
	//~ Begin UPropertyAnimatorCorePresetBase
	PROPERTYANIMATORCORE_API virtual bool IsPresetSupported(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator) const override;
	PROPERTYANIMATORCORE_API virtual bool IsPresetApplied(const UPropertyAnimatorCoreBase* InAnimator) const override;
	PROPERTYANIMATORCORE_API virtual bool ApplyPreset(UPropertyAnimatorCoreBase* InAnimator) override;
	PROPERTYANIMATORCORE_API virtual bool UnapplyPreset(UPropertyAnimatorCoreBase* InAnimator) override;
	PROPERTYANIMATORCORE_API virtual void CreatePreset(FName InName, const TArray<IPropertyAnimatorCorePresetable*>& InPresetableItems) override;
	PROPERTYANIMATORCORE_API virtual bool LoadPreset() override;
	//~ End UPropertyAnimatorCorePresetBase

	/** Called when this preset is applied on the animator */
	PROPERTYANIMATORCORE_API virtual void OnPresetApplied(UPropertyAnimatorCoreBase* InAnimator, const TSet<FPropertyAnimatorCoreData>& InProperties);

	/** Called when this preset is unapplied on the animator */
	virtual void OnPresetUnapplied(UPropertyAnimatorCoreBase* InAnimator, const TSet<FPropertyAnimatorCoreData>& InProperties) {}

	/** Get the preset properties for that actor */
	PROPERTYANIMATORCORE_API virtual void GetPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const;

	/** Get the preset properties for that actor but only supported ones for that animator */
	PROPERTYANIMATORCORE_API virtual void GetSupportedPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const;

	/** Gets the supported and applied properties for an animator */
	PROPERTYANIMATORCORE_API virtual void GetAppliedPresetProperties(const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutSupportedProperties, TSet<FPropertyAnimatorCoreData>& OutAppliedProperties);

protected:
	TMap<FString, TSharedRef<FPropertyAnimatorCorePresetArchive>> PropertyPresets;
};
