// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/Object.h"
#include "PropertyAnimatorCoreConverterBase.generated.h"

class UScriptStruct;
struct FPropertyAnimatorCoreConverterRuleBase;

/**
 * Abstract base class,
 * Children are used to convert from a type to another with optional rules,
 * Should remain transient and stateless
 */
UCLASS(MinimalAPI, Abstract, Transient)
class UPropertyAnimatorCoreConverterBase : public UObject
{
	GENERATED_BODY()

public:
	/** Checks if this converter support the conversion */
	virtual bool IsConversionSupported(const FPropertyBagPropertyDesc& InFromProperty, const FPropertyBagPropertyDesc& InToProperty) const
	{
		return false;
	}

	/** Converts from input bag to output bag with optional rules, returns true on success */
	virtual bool Convert(const FPropertyBagPropertyDesc& InFromProperty, const FInstancedPropertyBag& InFromBag, const FPropertyBagPropertyDesc& InToProperty, FInstancedPropertyBag& InToBag, const TInstancedStruct<FPropertyAnimatorCoreConverterRuleBase>& InRule)
	{
		return false;
	}

	/** Returns a struct descriptor for optional rules supported by this converter */
	virtual UScriptStruct* GetConversionRuleStruct() const
	{
		return nullptr;
	}
};