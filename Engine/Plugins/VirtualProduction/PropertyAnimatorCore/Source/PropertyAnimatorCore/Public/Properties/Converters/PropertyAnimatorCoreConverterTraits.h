// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyAnimatorCoreConverterTraits.generated.h"

/** Base struct for converter rules */
USTRUCT()
struct FPropertyAnimatorCoreConverterRuleBase
{
	GENERATED_BODY()
};

/** Helper to define a converter from a type to another along an optional struct for rules */
template<typename InFromType, typename InToType, typename InRuleType = void>
struct TValueConverterTraits
{
	static bool Convert(const InFromType& InFrom, InToType& OutTo, const InRuleType* InRule = nullptr)
	{
		// No converter declared for that conversion
		checkNoEntry();
		return false;
	}

	static UScriptStruct* GetRuleStruct()
	{
		return nullptr;
	}
};

UENUM(BlueprintType)
enum class EBoolConverterComparison : uint8
{
	Equal,
	NotEqual,
	Greater,
	Less,
	GreaterEqual,
	LessEqual,
};

USTRUCT(BlueprintType)
struct FBoolConverterCondition
{
	GENERATED_BODY()

	FBoolConverterCondition() {}
	explicit FBoolConverterCondition(EBoolConverterComparison InComparison, double InThreshold)
		: Comparison(InComparison)
		, Threshold(InThreshold)
	{}

	UPROPERTY(EditInstanceOnly, Category="Converter")
	EBoolConverterComparison Comparison = EBoolConverterComparison::NotEqual;

	UPROPERTY(EditInstanceOnly, Category="Converter")
	double Threshold = 0;

};

USTRUCT(BlueprintType)
struct FBoolConverterRule : public FPropertyAnimatorCoreConverterRuleBase
{
	GENERATED_BODY()

	FBoolConverterRule()
	{
		// Value != 0 is true else false
		TrueConditions.Add(FBoolConverterCondition());
	}

	UPROPERTY(EditInstanceOnly, Category="Converter")
	TArray<FBoolConverterCondition> TrueConditions;
};

template<>
struct TValueConverterTraits<float, bool, FBoolConverterRule>
{
	static bool Convert(const float& InFrom, bool& OutTo, const FBoolConverterRule* InRule = nullptr)
	{
		if (!InRule)
		{
			OutTo = !FMath::IsNearlyEqual(InFrom, 0.f);
		}
		else
		{
			OutTo = false;
			for (const FBoolConverterCondition& Condition : InRule->TrueConditions)
			{
				if (Condition.Comparison == EBoolConverterComparison::Equal && FMath::IsNearlyEqual(InFrom, Condition.Threshold))
				{
					OutTo = true;
				}
				else if (Condition.Comparison == EBoolConverterComparison::NotEqual && !FMath::IsNearlyEqual(InFrom, Condition.Threshold))
				{
					OutTo = true;
				}
				else if (Condition.Comparison == EBoolConverterComparison::Greater && InFrom > Condition.Threshold)
				{
					OutTo = true;
				}
				else if (Condition.Comparison == EBoolConverterComparison::Less && InFrom < Condition.Threshold)
				{
					OutTo = true;
				}
				else if (Condition.Comparison == EBoolConverterComparison::GreaterEqual && InFrom >= Condition.Threshold)
				{
					OutTo = true;
				}
				else if (Condition.Comparison == EBoolConverterComparison::LessEqual && InFrom <= Condition.Threshold)
				{
					OutTo = true;
				}
			}
		}

		return true;
	}

	static UScriptStruct* GetRuleStruct()
	{
		return FBoolConverterRule::StaticStruct();
	}
};

template<>
struct TValueConverterTraits<bool, float>
{
	static bool Convert(const bool& InFrom, float& OutTo, const void* InRule = nullptr)
	{
		OutTo = InFrom ? 1.f : 0.f;
		return true;
	}

	static UScriptStruct* GetRuleStruct()
	{
		return nullptr;
	}
};

template<>
struct TValueConverterTraits<double, bool, FBoolConverterRule>
{
	static bool Convert(const double& InFrom, bool& OutTo, const FBoolConverterRule* InRule = nullptr)
	{
		if (!InRule)
		{
			OutTo = !FMath::IsNearlyEqual(InFrom, 0);
		}
		else
		{
			OutTo = false;
			for (const FBoolConverterCondition& Condition : InRule->TrueConditions)
			{
				if (Condition.Comparison == EBoolConverterComparison::Equal && FMath::IsNearlyEqual(InFrom, Condition.Threshold))
				{
					OutTo = true;
				}
				else if (Condition.Comparison == EBoolConverterComparison::NotEqual && !FMath::IsNearlyEqual(InFrom, Condition.Threshold))
				{
					OutTo = true;
				}
				else if (Condition.Comparison == EBoolConverterComparison::Greater && InFrom > Condition.Threshold)
				{
					OutTo = true;
				}
				else if (Condition.Comparison == EBoolConverterComparison::Less && InFrom < Condition.Threshold)
				{
					OutTo = true;
				}
				else if (Condition.Comparison == EBoolConverterComparison::GreaterEqual && InFrom >= Condition.Threshold)
				{
					OutTo = true;
				}
				else if (Condition.Comparison == EBoolConverterComparison::LessEqual && InFrom <= Condition.Threshold)
				{
					OutTo = true;
				}
			}
		}

		return true;
	}

	static UScriptStruct* GetRuleStruct()
	{
		return FBoolConverterRule::StaticStruct();
	}
};

template<>
struct TValueConverterTraits<bool, double>
{
	static bool Convert(const bool& InFrom, double& OutTo, const void* InRule = nullptr)
	{
		OutTo = InFrom ? 1.0 : 0.0;
		return true;
	}

	static UScriptStruct* GetRuleStruct()
	{
		return nullptr;
	}
};

template<>
struct TValueConverterTraits<double, float>
{
	static bool Convert(const double& InFrom, float& OutTo, const void* InRule = nullptr)
	{
		OutTo = static_cast<float>(InFrom);
		return true;
	}

	static UScriptStruct* GetRuleStruct()
	{
		return nullptr;
	}
};

template<>
struct TValueConverterTraits<float, double>
{
	static bool Convert(const float& InFrom, double& OutTo, const void* InRule = nullptr)
	{
		OutTo = static_cast<double>(InFrom);
		return true;
	}

	static UScriptStruct* GetRuleStruct()
	{
		return nullptr;
	}
};

UENUM(BlueprintType)
enum class EInt32ConverterMethod : uint8
{
	Round,
	Floor,
	Ceil
};

USTRUCT(BlueprintType)
struct FInt32ConverterRule : public FPropertyAnimatorCoreConverterRuleBase
{
	GENERATED_BODY()

	UPROPERTY(EditInstanceOnly, Category="Converter")
	EInt32ConverterMethod Method = EInt32ConverterMethod::Round;
};

template<>
struct TValueConverterTraits<float, int32, FInt32ConverterRule>
{
	static bool Convert(const float& InFrom, int32& OutTo, const FInt32ConverterRule* InRule)
	{
		if (!InRule || InRule->Method == EInt32ConverterMethod::Round)
		{
			OutTo = FMath::RoundToInt(InFrom);
		}
		else if (InRule->Method == EInt32ConverterMethod::Floor)
		{
			OutTo = FMath::FloorToInt(InFrom);
		}
		else if (InRule->Method == EInt32ConverterMethod::Ceil)
		{
			OutTo = FMath::CeilToInt(InFrom);
		}

		return true;
	}

	static UScriptStruct* GetRuleStruct()
	{
		return FInt32ConverterRule::StaticStruct();
	}
};

template<>
struct TValueConverterTraits<int32, float>
{
	static bool Convert(const int32& InFrom, float& OutTo, const void* InRule = nullptr)
	{
		OutTo = static_cast<float>(InFrom);
		return true;
	}

	static UScriptStruct* GetRuleStruct()
	{
		return nullptr;
	}
};

template<>
struct TValueConverterTraits<double, int32, FInt32ConverterRule>
{
	static bool Convert(const double& InFrom, int32& OutTo, const FInt32ConverterRule* InRule)
	{
		if (!InRule)
		{
			OutTo = FMath::RoundToInt(InFrom);
		}
		else if (InRule->Method == EInt32ConverterMethod::Round)
		{
			OutTo = FMath::RoundToInt(InFrom);
		}
		else if (InRule->Method == EInt32ConverterMethod::Floor)
		{
			OutTo = FMath::FloorToInt(InFrom);
		}
		else if (InRule->Method == EInt32ConverterMethod::Ceil)
		{
			OutTo = FMath::CeilToInt(InFrom);
		}

		return true;
	}

	static UScriptStruct* GetRuleStruct()
	{
		return FInt32ConverterRule::StaticStruct();
	}
};

template<>
struct TValueConverterTraits<int32, double>
{
	static bool Convert(const int32& InFrom, double& OutTo, const void* InRule = nullptr)
	{
		OutTo = static_cast<double>(InFrom);
		return true;
	}

	static UScriptStruct* GetRuleStruct()
	{
		return nullptr;
	}
};

template<>
struct TValueConverterTraits<FString, FText>
{
	static bool Convert(const FString& InFrom, FText& OutTo, const void* InRule = nullptr)
	{
		OutTo = FText::FromString(InFrom);
		return true;
	}

	static UScriptStruct* GetRuleStruct()
	{
		return nullptr;
	}
};

template<>
struct TValueConverterTraits<FText, FString>
{
	static bool Convert(const FText& InFrom, FString& OutTo, const void* InRule = nullptr)
	{
		OutTo = InFrom.ToString();
		return true;
	}

	static UScriptStruct* GetRuleStruct()
	{
		return nullptr;
	}
};
