// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Properties/PropertyAnimatorCoreData.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/Field.h"
#include "PropertyAnimatorCoreHandlerBase.generated.h"

/**
 * Abstract base class,
 * Children are used to get and set value for property type,
 * Should remain transient and stateless
 */
UCLASS(MinimalAPI, Abstract, Transient)
class UPropertyAnimatorCoreHandlerBase : public UObject
{
	GENERATED_BODY()

public:
	/** Checks whether this handlers supports the property */
	virtual bool IsPropertySupported(const FPropertyAnimatorCoreData& InPropertyData) const
	{
		return false;
	}

	/** Retrieves the value from this property within owner, value will be in the property bag */
	virtual bool GetValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue)
	{
		return false;
	}

	/** Sets the value to this property within owner, value is in the property bag */
	virtual bool SetValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
	{
		return false;
	}

	/** Whether Add and Subtract operations can be performed on the property value */
	virtual bool IsAdditiveSupported() const
	{
		return false;
	}

	/** Adds the value to this property within owner, delta value is in the property bag */
	virtual bool AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
	{
		return false;
	}

	/** Subtract the value to this property within owner, delta value is in the property bag */
	virtual bool SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue)
	{
		return false;
	}

	/** Adds ValueA to ValueB and stores the result in OutValue */
	virtual bool AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValueA, const FInstancedPropertyBag& InValueB, FInstancedPropertyBag& OutValue)
	{
		return false;
	}

	/** Subtracts ValueB from ValueA and stores the result in OutValue */
	virtual bool SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValueA, const FInstancedPropertyBag& InValueB, FInstancedPropertyBag& OutValue)
	{
		return false;
	}

	virtual bool AddValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue, FInstancedPropertyBag& OutValue)
	{
		return false;
	}

	virtual bool SubtractValue(const FPropertyAnimatorCoreData& InPropertyData, const FInstancedPropertyBag& InValue, FInstancedPropertyBag& OutValue)
	{
		return false;
	}

	/** Get the default value for this property, init to zero */
	virtual bool GetDefaultValue(const FPropertyAnimatorCoreData& InPropertyData, FInstancedPropertyBag& OutValue)
	{
		return false;
	}
};

