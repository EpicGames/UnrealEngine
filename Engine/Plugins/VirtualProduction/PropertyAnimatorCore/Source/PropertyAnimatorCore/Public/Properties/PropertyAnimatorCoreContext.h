// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Converters/PropertyAnimatorCoreConverterTraits.h"
#include "PropertyAnimatorCoreData.h"
#include "Presets/PropertyAnimatorCorePresetable.h"
#include "Presets/PropertyAnimatorCorePresetBase.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/Object.h"
#include "PropertyAnimatorCoreContext.generated.h"

class UPropertyAnimatorCoreBase;
class UPropertyAnimatorCoreConverterBase;
class UPropertyAnimatorCoreResolver;
class UScriptStruct;
struct FPropertyAnimatorCoreConverterRuleBase;

/** Mode supported for properties value */
UENUM(BlueprintType)
enum class EPropertyAnimatorCoreMode : uint8
{
	/** Set the property value directly */
	Absolute,
	/** Add value on the existing property value */
	Additive
};

/** Context for properties linked to an animator */
UCLASS(MinimalAPI, BlueprintType)
class UPropertyAnimatorCoreContext : public UObject, public IPropertyAnimatorCorePresetable
{
	GENERATED_BODY()

	friend class UPropertyAnimatorCoreBase;
	friend class FPropertyAnimatorCoreEditorContextTypeCustomization;

public:
	static FName GetAnimatedPropertyName();

	const FPropertyAnimatorCoreData& GetAnimatedProperty() const
	{
		return AnimatedProperty;
	}

	UPropertyAnimatorCoreBase* GetAnimator() const;

	/** Get the handler responsible for this property type */
	UPropertyAnimatorCoreHandlerBase* GetHandler() const;

	/** Get the active resolver for this property if any */
	UPropertyAnimatorCoreResolver* GetResolver() const;

	PROPERTYANIMATORCORE_API void SetAnimated(bool bInAnimated);
	bool IsAnimated() const
	{
		return bAnimated;
	}

	PROPERTYANIMATORCORE_API void SetMagnitude(float InMagnitude);
	float GetMagnitude() const
	{
		return Magnitude;
	}

	PROPERTYANIMATORCORE_API void SetTimeOffset(double InOffset);
	double GetTimeOffset() const
	{
		return TimeOffset;
	}

	PROPERTYANIMATORCORE_API void SetMode(EPropertyAnimatorCoreMode InMode);
	EPropertyAnimatorCoreMode GetMode() const
	{
		return Mode;
	}

	PROPERTYANIMATORCORE_API void SetConverterClass(const TSubclassOf<UPropertyAnimatorCoreConverterBase>& InConverterClass);
	TSubclassOf<UPropertyAnimatorCoreConverterBase> GetConverterClass() const
	{
		return ConverterClass;
	}

	/** Get converter rule if any */
	template <typename InRuleType
		UE_REQUIRES(TModels_V<CStaticStructProvider, InRuleType>)>
	InRuleType* GetConverterRule()
	{
		return static_cast<InRuleType*>(GetConverterRulePtr(InRuleType::StaticStruct()));
	}

	/** Called when the owner has changed and we want to update the animated property */
	bool ResolvePropertyOwner(AActor* InNewOwner);

	/** Evaluates a property within this context based on animator result */
	virtual bool EvaluateProperty(const FPropertyAnimatorCoreData& InProperty, const FInstancedPropertyBag& InAnimatorResult, FInstancedPropertyBag& OutEvaluatedValues)
	{
		return false;
	}

	//~ Begin IPropertyAnimatorCorePresetable
	PROPERTYANIMATORCORE_API virtual bool ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue) override;
	PROPERTYANIMATORCORE_API virtual bool ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const override;
	//~ End IPropertyAnimatorCorePresetable

protected:
	//~ Begin UObject
	PROPERTYANIMATORCORE_API virtual void PostLoad() override;
#if WITH_EDITOR
	PROPERTYANIMATORCORE_API virtual void PreEditChange(FProperty* InPropertyAboutToChange) override;
	PROPERTYANIMATORCORE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	/** Called once, when the property is linked to this context */
	PROPERTYANIMATORCORE_API virtual void OnAnimatedPropertyLinked();

	/** Called when the animated property owner is updated */
	virtual void OnAnimatedPropertyOwnerUpdated(UObject* InPreviousOwner, UObject* InNewOwner) {}

private:
	void ConstructInternal(const FPropertyAnimatorCoreData& InProperty);
	void SetAnimatedPropertyOwner(UObject* InNewOwner);

	PROPERTYANIMATORCORE_API void* GetConverterRulePtr(const UScriptStruct* InStruct);

	void CheckEditConditions();
	void CheckEditMagnitude();
	void CheckEditTimeOffset();
	void CheckEditMode();
	void CheckEditConverterRule();
	void CheckEditResolver();

	void OnAnimatedChanged();
	void OnModeChanged();

	/** Sets the evaluation result for the resolved property */
	PROPERTYANIMATORCORE_API void CommitEvaluationResult(const FPropertyAnimatorCoreData& InResolvedProperty, FInstancedPropertyBag& InEvaluatedValues);

	/** Use this to resolve virtual linked property */
	PROPERTYANIMATORCORE_API TArray<FPropertyAnimatorCoreData> ResolveProperty(bool bInForEvaluation) const;

	/** Restore property based on mode */
	void Restore();

	/** Allocate and save properties */
	void Save();

	bool IsResolvable() const;
	bool IsConverted() const;

	/** Animation is enabled for this property */
	UPROPERTY(EditInstanceOnly, Setter="SetAnimated", Getter="IsAnimated", Category="Animator", meta=(AllowPrivateAccess="true"))
	bool bAnimated = true;

	/** Edit condition for Magnitude */
	UPROPERTY(Transient)
	bool bEditMagnitude = true;

	/** Edit condition for TimeOffset */
	UPROPERTY(Transient)
	bool bEditTimeOffset = true;

	/** Edit condition for modes */
	UPROPERTY(Transient)
	bool bEditMode = true;

	/** Edit condition for converter rule */
	UPROPERTY(Transient)
	bool bEditConverterRule = false;

	/** Edit condition for property resolver */
	UPROPERTY(Transient)
	bool bEditResolver = true;

	/** Magnitude of the effect on this property */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(ClampMin="0.0", ClampMax="1.0", HideEditConditionToggle, EditCondition="bEditMagnitude", EditConditionHides, AllowPrivateAccess="true"))
	float Magnitude = 1.f;

	/** Time offset to evaluate this property */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(Units=Seconds, HideEditConditionToggle, EditCondition="bEditTimeOffset", EditConditionHides, AllowPrivateAccess="true"))
	double TimeOffset = 0;

	/** Current mode used for this property */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(HideEditConditionToggle, EditCondition="bEditMode", EditConditionHides, AllowPrivateAccess="true"))
	EPropertyAnimatorCoreMode Mode = EPropertyAnimatorCoreMode::Absolute;

	/** If a converter is used, rules may be used to convert the property */
	UPROPERTY(EditInstanceOnly, Category="Animator", NoClear, meta=(HideEditConditionToggle, EditCondition="bEditConverterRule", EditConditionHides, AllowPrivateAccess="true"))
	TInstancedStruct<FPropertyAnimatorCoreConverterRuleBase> ConverterRule;

	/** Custom resolver for the property */
	UPROPERTY(VisibleInstanceOnly, NoClear, Export, Instanced, DisplayName="Range", Category="Animator", meta=(EditCondition="bEditResolver", EditConditionHides, HideEditConditionToggle, AllowPrivateAccess="true"))
	TObjectPtr<UPropertyAnimatorCoreResolver> Resolver;

	/** Store original property values for resolved properties */
	UPROPERTY(NonTransactional)
	FInstancedPropertyBag OriginalPropertyValues;

	/** Store delta property values for resolved properties */
	UPROPERTY(NonTransactional)
	FInstancedPropertyBag DeltaPropertyValues;

	/** Converter class used for this property */
	UPROPERTY()
	TSubclassOf<UPropertyAnimatorCoreConverterBase> ConverterClass;

	/** Used to access property value and update it */
	UPROPERTY(Transient)
	TWeakObjectPtr<UPropertyAnimatorCoreHandlerBase> HandlerWeak;

	/** Animated property linked to this options */
	UPROPERTY()
	FPropertyAnimatorCoreData AnimatedProperty;
};