// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Misc/Optional.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/FieldPath.h"
#include "UObject/WeakFieldPtr.h"

#include "PropertyAnimatorCoreData.generated.h"

class UPropertyAnimatorCoreHandlerBase;
class UPropertyAnimatorCoreResolver;

/** Serializable struct that contains the property and the owner with accessors */
USTRUCT(BlueprintType)
struct FPropertyAnimatorCoreData
{
	GENERATED_BODY()

	FPropertyAnimatorCoreData() = default;

	/** Takes the owner, the member property and the inner property inside */
	PROPERTYANIMATORCORE_API explicit FPropertyAnimatorCoreData(UObject* InObject, FProperty* InMemberProperty, FProperty* InProperty, TSubclassOf<UPropertyAnimatorCoreResolver> InResolverClass = nullptr);

	/** Take the owner and the full property chain, from member property to inner property */
	PROPERTYANIMATORCORE_API explicit FPropertyAnimatorCoreData(UObject* InObject, const TArray<FProperty*>& InChainProperties, TSubclassOf<UPropertyAnimatorCoreResolver> InResolverClass = nullptr);

	/** Take the owner, the property chain until the inner property and lastly the inner property */
	PROPERTYANIMATORCORE_API explicit FPropertyAnimatorCoreData(UObject* InObject, const TArray<FProperty*>& InChainProperties, FProperty* InProperty, TSubclassOf<UPropertyAnimatorCoreResolver> InResolverClass = nullptr);

	/** Takes an actor and a locator path, tries to resolve property and owner */
	PROPERTYANIMATORCORE_API explicit FPropertyAnimatorCoreData(AActor* InActor, const FString& InPropertyLocatorPath);

	/** Is this a resolvable property that uses a custom resolver */
	PROPERTYANIMATORCORE_API bool IsResolvable() const;

	/** Get the linked property resolver for resolvable properties */
	PROPERTYANIMATORCORE_API UPropertyAnimatorCoreResolver* GetPropertyResolver() const;

	/** Get the property resolver class */
	TSubclassOf<UPropertyAnimatorCoreResolver> GetPropertyResolverClass() const;

	/** Checks if the property is resolved and can be used */
	bool IsResolved() const
	{
		return OwnerWeak.IsValid() && GetMemberProperty();
	}

	friend uint32 GetTypeHash(const FPropertyAnimatorCoreData& InItem)
	{
		return GetTypeHash(InItem.LocatorPath);
	}

	bool operator==(const FPropertyAnimatorCoreData& InOther) const
	{
		return LocatorPath == InOther.LocatorPath;
	}

	/** Returns the owning actor from this owner object */
	PROPERTYANIMATORCORE_API AActor* GetOwningActor() const;

	/** Returns the owning component from this owner object if any */
	PROPERTYANIMATORCORE_API UActorComponent* GetOwningComponent() const;

	/** The owner object of the member property */
	UObject* GetOwner() const
	{
		return OwnerWeak.Get();
	}

	TWeakObjectPtr<UObject> GetOwnerWeak() const
	{
		return OwnerWeak;
	}

	/** Returns chain of owner until it reaches StopOuter */
	TArray<UObject*> GetOuters(const UObject* InStopOuter) const;

	/** The member property of the owner, top property inside the owner itself */
	FProperty* GetMemberProperty() const
	{
		return !ChainProperties.IsEmpty() ? ChainProperties[0].Get() : nullptr;
	}

	/** The last property in the chain, could be member property if there is only one */
	FProperty* GetLeafProperty() const
	{
		return !ChainProperties.IsEmpty() ? ChainProperties.Last().Get() : nullptr;
	}

	/** The friendly display name from the member property to the inner property */
	PROPERTYANIMATORCORE_API FString GetPropertyDisplayName() const;

	/**
	 * DEPRECATED, use GetLocatorPath instead
	 * The full path of a property with its owner
	 */
	PROPERTYANIMATORCORE_API FString GetPathHash() const;

	/** The path to retrieve owner and property on any actor */
	PROPERTYANIMATORCORE_API FString GetLocatorPath() const;

	/** Hash used to store property, based on the locator path */
	PROPERTYANIMATORCORE_API FName GetLocatorPathHash() const;

	/** The member property name */
	PROPERTYANIMATORCORE_API FName GetMemberPropertyName() const;

	PROPERTYANIMATORCORE_API FName GetMemberPropertyTypeName() const;

	/** The leaf property name */
	PROPERTYANIMATORCORE_API FName GetLeafPropertyName() const;

	PROPERTYANIMATORCORE_API FName GetLeafPropertyTypeName() const;

	/** The chain properties from member to inner property */
	PROPERTYANIMATORCORE_API TArray<FProperty*> GetChainProperties() const;

	/** The chain property names from member to inner property */
	PROPERTYANIMATORCORE_API TArray<FName> GetChainPropertyNames() const;

	/** Checks if the property is settable via setter */
	PROPERTYANIMATORCORE_API bool HasSetter() const;

	/** Checks if we contain this other property directly */
	bool IsParentOf(const FPropertyAnimatorCoreData& InOtherProperty) const;

	/** Checks if we are contained inside this other property directly */
	bool IsChildOf(const FPropertyAnimatorCoreData& InOtherProperty) const;

	/** Checks if we own this other nested property */
	bool IsOwning(const FPropertyAnimatorCoreData& InOtherProperty) const;

	/** Returns true if the owner or the property is transient */
	bool IsTransient() const;

	/** Tries to find, based on this property, the direct child of other property */
	PROPERTYANIMATORCORE_API TOptional<FPropertyAnimatorCoreData> GetChildOf(const FPropertyAnimatorCoreData& InOtherProperty) const;

	/** Returns the parent of this property if there is one */
	TOptional<FPropertyAnimatorCoreData> GetParent() const;

	/** Returns the top most parent / member property if there is one */
	PROPERTYANIMATORCORE_API TOptional<FPropertyAnimatorCoreData> GetRootParent() const;

	/** Checks if this property is of a specific type */
	template<typename InPropertyClass
		UE_REQUIRES(std::is_base_of_v<FProperty, InPropertyClass>)>
	bool IsA() const
	{
		if (const FProperty* LeafProperty = GetLeafProperty())
		{
			return LeafProperty->IsA(InPropertyClass::StaticClass());
		}

		return false;
	}

	/** Gets the children of this property */
	PROPERTYANIMATORCORE_API TArray<FPropertyAnimatorCoreData> GetChildrenProperties(int32 InDepthSearch = 3) const;

	/** Checks if this property contains a specific type */
	template<typename InPropertyClass
		UE_REQUIRES(std::is_base_of_v<FProperty, InPropertyClass>)>
	bool HasA() const
	{
		for (const FPropertyAnimatorCoreData& ChildProperty : GetChildrenProperties())
		{
			if (ChildProperty.IsA<InPropertyClass>())
			{
				return true;
			}
		}

		return false;
	}

	/**
	 * Gets the value ptr for this property
	 * will use getter if available, otherwise directly access the property address
	 */
	template <typename OutValueType>
	void GetPropertyValuePtr(OutValueType* OutValue) const
	{
		GetPropertyValuePtrInternal(OutValue);
	}

	/**
	 * Sets the value ptr for this property,
	 * will use setter if available, otherwise directly set the property address
	 */
	template <typename InValueType>
	void SetPropertyValuePtr(InValueType* InValue) const
	{
		SetPropertyValuePtrInternal(InValue);
	}

	/** Gets the property handler to perform operation on property without knowing the type */
	UPropertyAnimatorCoreHandlerBase* GetPropertyHandler() const;

	/** Internal use only, (re)creates the property path, locator path, display name */
	void GeneratePropertyPath();

private:
	static void CopyPropertyValue(const FProperty* InProperty, const void* InSrc, void* OutDest);

	static FName GetPropertyTypeName(const FProperty* InProperty);

	/** Internal use only, get property value */
	void GetPropertyValuePtrInternal(void* OutValue) const;

	/** Internal use only, set property value */
	void SetPropertyValuePtrInternal(const void* InValue) const;

	/** Internal use only, set leaf property value */
	void SetLeafPropertyValuePtrInternal(void* InContainer, const void* InValue) const;

	/** Internal use only, get leaf property value */
	void GetLeafPropertyValuePtrInternal(const void* InContainer, void* OutValue) const;

	/** Uses chained properties to resolve from container to */
	void* ContainerToValuePtr(const void* InContainer, int32 InStartPropertyIndex) const;

	/** Tries to find setter function for this property */
	bool FindSetterFunctions();

	/** Creates a single path that contains resolver, owner and properties for (de)serialization */
	void GeneratePropertyLocatorPath();

	/** Creates the display name for the underlying property */
	void GeneratePropertyDisplayName();

	/** Owner of the property */
	UPROPERTY()
	TWeakObjectPtr<UObject> OwnerWeak;

	/** The cached friendly display name of the property we are controlling */
	UPROPERTY(Transient)
	FString PropertyDisplayName;

	/**
	 * DEPRECATED, use locator path instead,
	 * Previously used to quickly compare struct of this type, replaced by LocatorPath */
	UPROPERTY()
	FString PathHash;

	/** Property locator path to find the property and owner on any actor */
	UPROPERTY()
	FString LocatorPath;

	/** Chain from member property to inner property */
	UPROPERTY()
	TArray<TFieldPath<FProperty>> ChainProperties;

	/** Matching Setter ufunction found when no setter is specified */
	UPROPERTY(Transient)
	TWeakObjectPtr<UFunction> SetterFunctionWeak;

	UPROPERTY(Transient)
	bool bSetterFunctionCached = false;

	/** Used by virtual properties to resolve */
	UPROPERTY()
	TSubclassOf<UPropertyAnimatorCoreResolver> PropertyResolverClass;

	/** Used by properties to get/set value with a property bag without knowing the underlying type */
	UPROPERTY(Transient)
	TObjectPtr<UPropertyAnimatorCoreHandlerBase> PropertyHandler;
};