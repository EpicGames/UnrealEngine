// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Presets/PropertyAnimatorCorePresetable.h"
#include "Properties/PropertyAnimatorCoreData.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "PropertyAnimatorCoreResolver.generated.h"

/**
 * Base class to find properties hidden or not reachable,
 * allows to discover resolvable properties for specific actors/components/objects
 * that we cannot reach or are transient, will be resolved when needed
 */
UCLASS(MinimalAPI, Abstract)
class UPropertyAnimatorCoreResolver : public UObject, public IPropertyAnimatorCorePresetable
{
	GENERATED_BODY()

public:
	UPropertyAnimatorCoreResolver()
		: UPropertyAnimatorCoreResolver(NAME_None)
	{}

	UPropertyAnimatorCoreResolver(FName InResolverName)
		: ResolverName(InResolverName)
	{}

	/** Tries to resolve old property against new one */
	virtual bool FixUpProperty(FPropertyAnimatorCoreData& InOldProperty)
	{
		return false;
	}

	/** Get template properties based on context */
	virtual void GetTemplateProperties(UObject* InContext, TSet<FPropertyAnimatorCoreData>& OutProperties, const TArray<FName>* InSearchPath = nullptr) {}

	/** Called when we actually need the underlying properties of a template property */
	virtual void ResolveTemplateProperties(const FPropertyAnimatorCoreData& InTemplateProperty, TArray<FPropertyAnimatorCoreData>& OutProperties, bool bInForEvaluation) {}

	FName GetResolverName() const
	{
		return ResolverName;
	}

	//~ Begin IPropertyAnimatorCorePresetable
	PROPERTYANIMATORCORE_API virtual bool ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue) override;
	PROPERTYANIMATORCORE_API virtual bool ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const override;
	//~ End IPropertyAnimatorCorePresetable

private:
	UPROPERTY()
	FName ResolverName = NAME_None;
};
