// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneAnimatorTypes.h"
#include "MovieSceneSection.h"
#include "MovieSceneAnimatorSection.generated.h"

/** Movie scene section for a sequencer animator track */
UCLASS(MinimalAPI)
class UMovieSceneAnimatorSection : public UMovieSceneSection
{
	GENERATED_BODY()

public:
	UMovieSceneAnimatorSection();

	PROPERTYANIMATORCORE_API void SetEvalTimeMode(EMovieSceneAnimatorEvalTimeMode InMode);

	EMovieSceneAnimatorEvalTimeMode GetEvalTimeMode() const
	{
		return EvalTimeMode;
	}

	PROPERTYANIMATORCORE_API void SetCustomStartTime(double InTime);

	double GetCustomStartTime() const
	{
		return CustomStartTime;
	}

	PROPERTYANIMATORCORE_API void SetCustomEndTime(double InTime);

	double GetCustomEndTime() const
	{
		return CustomEndTime;
	}

protected:
	UPROPERTY(EditAnywhere, Setter, Getter, Category="Animator")
	EMovieSceneAnimatorEvalTimeMode EvalTimeMode = EMovieSceneAnimatorEvalTimeMode::Section;

	UPROPERTY(EditAnywhere, Setter, Getter, Category="Animator", meta=(EditCondition="EvalTimeMode == EMovieSceneAnimatorEvalTimeMode::Custom"))
	double CustomStartTime = 0.0;

	UPROPERTY(EditAnywhere, Setter, Getter, Category="Animator", meta=(EditCondition="EvalTimeMode == EMovieSceneAnimatorEvalTimeMode::Custom"))
	double CustomEndTime = 1.0;
};
