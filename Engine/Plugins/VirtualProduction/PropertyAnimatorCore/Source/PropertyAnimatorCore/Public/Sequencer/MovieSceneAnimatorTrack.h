// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneAnimatorTrack.generated.h"

/** Movie scene track used to drive animator with sequencer */
UCLASS(MinimalAPI)
class UMovieSceneAnimatorTrack : public UMovieSceneNameableTrack, public IMovieSceneTrackTemplateProducer
{
	GENERATED_BODY()

public:
	UMovieSceneAnimatorTrack();

	//~ Begin UMovieSceneTrack
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> InSectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual void AddSection(UMovieSceneSection& InSection) override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool HasSection(const UMovieSceneSection& InSection) const override;
	virtual bool IsEmpty() const override;
	virtual void RemoveAllAnimationData() override;
	virtual void RemoveSection(UMovieSceneSection& InSection) override;
	virtual void RemoveSectionAt(int32 InSectionIndex) override;
#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
#endif
	//~ End UMovieSceneTrack

	//~ Begin IMovieSceneTrackTemplateProducer
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;
	//~ End IMovieSceneTrackTemplateProducer

private:
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;
};
