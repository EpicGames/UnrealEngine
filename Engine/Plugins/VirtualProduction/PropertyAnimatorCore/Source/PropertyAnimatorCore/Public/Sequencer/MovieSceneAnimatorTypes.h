// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MovieSceneAnimatorTypes.generated.h"

class UMovieSceneAnimatorSection;

/** Enumerates all possible ways of interpreting time */
UENUM()
enum class EMovieSceneAnimatorEvalTimeMode : uint8
{
	/** The sequence time, if the section has an offset, it won't matter */
	Sequence,
	/** The section time, takes into account any offset the section has */
	Section,
	/** Custom provided time, interpolate between start and end time based on progress */
	Custom
};

struct FMovieSceneAnimatorSectionData
{
	EMovieSceneAnimatorEvalTimeMode EvalTimeMode = EMovieSceneAnimatorEvalTimeMode::Sequence;

	double CustomStartTime;

	double CustomEndTime;

	/** The section object to get easing from */
	const UMovieSceneAnimatorSection* Section = nullptr;
};
