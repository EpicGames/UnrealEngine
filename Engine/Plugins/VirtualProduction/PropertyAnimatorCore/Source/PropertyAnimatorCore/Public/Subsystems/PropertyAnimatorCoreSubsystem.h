// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Presets/PropertyAnimatorCorePresetBase.h"
#include "Properties/PropertyAnimatorCoreData.h"
#include "Subsystems/EngineSubsystem.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "PropertyAnimatorCoreSubsystem.generated.h"

class UFunction;
class UPropertyAnimatorCoreConverterBase;
class UPropertyAnimatorCoreBase;
class UPropertyAnimatorCoreHandlerBase;
class UPropertyAnimatorCorePresetBase;
class UPropertyAnimatorCoreResolver;
class UPropertyAnimatorCoreTimeSourceBase;

/** This subsystem handle all property animators */
UCLASS()
class UPropertyAnimatorCoreSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

	friend class UPropertyAnimatorCoreComponent;

public:
	/** Get this subsystem instance */
	PROPERTYANIMATORCORE_API static UPropertyAnimatorCoreSubsystem* Get();

	//~ Begin USubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End USubsystem

	/** Register the property animator class to allow its usage */
	PROPERTYANIMATORCORE_API bool RegisterAnimatorClass(const UClass* InAnimatorClass);

	/** Unregister the property animator class to disallow its usage */
	PROPERTYANIMATORCORE_API bool UnregisterAnimatorClass(const UClass* InAnimatorClass);

	/** Checks if the property animator class is already registered */
	PROPERTYANIMATORCORE_API bool IsAnimatorClassRegistered(const UClass* InAnimatorClass) const;

	/** Gets the animator CDO registered from the class */
	UPropertyAnimatorCoreBase* GetAnimatorRegistered(const UClass* InAnimatorClass) const;

	/** Returns true if any animator is able to control that property or nested otherwise false */
	PROPERTYANIMATORCORE_API bool IsPropertySupported(const FPropertyAnimatorCoreData& InPropertyData, bool bInCheckNestedProperties = true) const;

	/** Find all animators linked to the property */
	PROPERTYANIMATORCORE_API TSet<UPropertyAnimatorCoreBase*> GetPropertyLinkedAnimators(const FPropertyAnimatorCoreData& InPropertyData) const;

	/** Returns a set of existing property animator objects in owner that supports that property */
	PROPERTYANIMATORCORE_API TSet<UPropertyAnimatorCoreBase*> GetExistingAnimators(const FPropertyAnimatorCoreData& InPropertyData) const;
	PROPERTYANIMATORCORE_API TSet<UPropertyAnimatorCoreBase*> GetExistingAnimators(const AActor* InActor) const;

	/** Returns a set of property animator CDO that supports that property */
	PROPERTYANIMATORCORE_API TSet<UPropertyAnimatorCoreBase*> GetAvailableAnimators(const FPropertyAnimatorCoreData* InPropertyData) const;
	PROPERTYANIMATORCORE_API TSet<UPropertyAnimatorCoreBase*> GetAvailableAnimators() const;

	/** Register the property handler class to allow its usage */
	PROPERTYANIMATORCORE_API bool RegisterHandlerClass(const UClass* InHandlerClass);

	/** Unregister the property handler class to disallow its usage */
	PROPERTYANIMATORCORE_API bool UnregisterHandlerClass(const UClass* InHandlerClass);

	/** Checks if the property handler class is already registered */
	PROPERTYANIMATORCORE_API bool IsHandlerClassRegistered(const UClass* InHandlerClass) const;

	/** Gets a property handler for this property */
	UPropertyAnimatorCoreHandlerBase* GetHandler(const FPropertyAnimatorCoreData& InPropertyData) const;

	/** Register a resolver for custom properties */
	PROPERTYANIMATORCORE_API bool RegisterResolverClass(const UClass* InResolverClass);

	/** Unregister a resolver */
	PROPERTYANIMATORCORE_API bool UnregisterResolverClass(const UClass* InResolverClass);

	UPropertyAnimatorCoreResolver* FindResolverByName(FName InResolverName);

	UPropertyAnimatorCoreResolver* FindResolverByClass(const UClass* InResolverClass);

	/** Is this resolver registered */
	PROPERTYANIMATORCORE_API bool IsResolverClassRegistered(const UClass* InResolverClass) const;

	/** Register a time source class to control clock for animators */
	PROPERTYANIMATORCORE_API bool RegisterTimeSourceClass(UClass* InTimeSourceClass);

	/** Unregister a time source class */
	PROPERTYANIMATORCORE_API bool UnregisterTimeSourceClass(UClass* InTimeSourceClass);

	/** Check time source class is registered */
	PROPERTYANIMATORCORE_API bool IsTimeSourceClassRegistered(UClass* InTimeSourceClass) const;

	/** Get all time source names available */
	TArray<FName> GetTimeSourceNames() const;

	/** Get all time sources available */
	TArray<UPropertyAnimatorCoreTimeSourceBase*> GetTimeSources() const;

	/** Get a registered time source using its name */
	UPropertyAnimatorCoreTimeSourceBase* GetTimeSource(FName InTimeSourceName) const;

	/** Create a new time source for an animator */
	UPropertyAnimatorCoreTimeSourceBase* CreateNewTimeSource(FName InTimeSourceName, UObject* InOwner);

	/** Register a preset class */
	PROPERTYANIMATORCORE_API bool RegisterPresetClass(const UClass* InPresetClass);

	/** Unregister a preset class */
	PROPERTYANIMATORCORE_API bool UnregisterPresetClass(const UClass* InPresetClass);

	/** Is this preset class registered */
	PROPERTYANIMATORCORE_API bool IsPresetClassRegistered(const UClass* InPresetClass) const;

	/** Get all registered preset available */
	PROPERTYANIMATORCORE_API TSet<UPropertyAnimatorCorePresetBase*> GetAvailablePresets(TSubclassOf<UPropertyAnimatorCorePresetBase> InPresetClass) const;

	/** Gets all supported presets for a specific animator and actor */
	PROPERTYANIMATORCORE_API TSet<UPropertyAnimatorCorePresetBase*> GetSupportedPresets(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSubclassOf<UPropertyAnimatorCorePresetBase> InPresetClass) const;

	PROPERTYANIMATORCORE_API bool RegisterSetterResolver(FName InPropertyName, TFunction<UFunction*(const UObject*)>&& InFunction);

	PROPERTYANIMATORCORE_API bool UnregisterSetterResolver(FName InPropertyName);

	PROPERTYANIMATORCORE_API bool IsSetterResolverRegistered(FName InPropertyName) const;

	UFunction* ResolveSetter(FName InPropertyName, const UObject* InOwner);

	/** Register a converter class */
	PROPERTYANIMATORCORE_API bool RegisterConverterClass(const UClass* InConverterClass);

	/** Unregister a converter class */
	PROPERTYANIMATORCORE_API bool UnregisterConverterClass(const UClass* InConverterClass);

	/** Is this converter class registered */
	PROPERTYANIMATORCORE_API bool IsConverterClassRegistered(const UClass* InConverterClass);

	/** Checks if any converter supports the type conversion */
	PROPERTYANIMATORCORE_API bool IsConversionSupported(const FPropertyBagPropertyDesc& InFromProperty, const FPropertyBagPropertyDesc& InToProperty);

	/** Finds suitable converters for a type conversion */
	PROPERTYANIMATORCORE_API TSet<UPropertyAnimatorCoreConverterBase*> GetSupportedConverters(const FPropertyBagPropertyDesc& InFromProperty, const FPropertyBagPropertyDesc& InToProperty) const;

	/** Registers a property alias by using a property identifier and property name, property identifier should be like Type.InnerType.PropertyName */
	bool RegisterPropertyAlias(const FString& InPropertyIdentifier, const FString& InAliasPropertyName);

	/** Unregisters a property alias */
	bool UnregisterPropertyAlias(const FString& InPropertyIdentifier);

	/** Finds a property alias registered or none */
	FString FindPropertyAlias(const FString& InPropertyIdentifier) const;

	/** Create an animator of specific class for an actor */
	PROPERTYANIMATORCORE_API UPropertyAnimatorCoreBase* CreateAnimator(AActor* InActor, const UClass* InAnimatorClass, UPropertyAnimatorCorePresetBase* InPreset = nullptr, bool bInTransact = false) const;

	/** Create animators of specific class for actors */
	PROPERTYANIMATORCORE_API TSet<UPropertyAnimatorCoreBase*> CreateAnimators(const TSet<AActor*>& InActors, const UClass* InAnimatorClass, UPropertyAnimatorCorePresetBase* InPreset = nullptr, bool bInTransact = false) const;

	/** Create animators of specific class for actors with presets */
	PROPERTYANIMATORCORE_API TSet<UPropertyAnimatorCoreBase*> CreateAnimators(const TSet<AActor*>& InActors, const UClass* InAnimatorClass, const TSet<UPropertyAnimatorCorePresetBase*>& InPresets, bool bInTransact = false) const;

	/** Clone animators onto an actor */
	PROPERTYANIMATORCORE_API TSet<UPropertyAnimatorCoreBase*> CloneAnimators(const TSet<UPropertyAnimatorCoreBase*>& InAnimators, AActor* InTargetActor, bool bInTransact = false) const;

	/** Removes a animator bound to an owner */
	PROPERTYANIMATORCORE_API bool RemoveAnimator(UPropertyAnimatorCoreBase* InAnimator, bool bInTransact = false) const;

	/** Removes animators from their owner */
	PROPERTYANIMATORCORE_API bool RemoveAnimators(const TSet<UPropertyAnimatorCoreBase*>& InAnimators, bool bInTransact = false) const;

	/** Removes animator components from their actor */
	PROPERTYANIMATORCORE_API bool RemoveAnimatorComponents(const TSet<UPropertyAnimatorCoreComponent*>& InComponents, bool bInTransact = false) const;

	/** Apply a preset on an existing animator */
	PROPERTYANIMATORCORE_API bool ApplyAnimatorPreset(UPropertyAnimatorCoreBase* InAnimator, UPropertyAnimatorCorePresetBase* InPreset, bool bInTransact = false);

	/** Unapply a preset from an existing animator */
	PROPERTYANIMATORCORE_API bool UnapplyAnimatorPreset(UPropertyAnimatorCoreBase* InAnimator, UPropertyAnimatorCorePresetBase* InPreset, bool bInTransact = false);

	/** Link a property to an existing animator */
	PROPERTYANIMATORCORE_API bool LinkAnimatorProperty(UPropertyAnimatorCoreBase* InAnimator, FPropertyAnimatorCoreData& InProperty, bool bInTransact = false);
	PROPERTYANIMATORCORE_API bool LinkAnimatorProperties(UPropertyAnimatorCoreBase* InAnimator, const TSet<FPropertyAnimatorCoreData>& InProperties, bool bInTransact = false);

	/** Unlink a property from an existing animator */
	PROPERTYANIMATORCORE_API bool UnlinkAnimatorProperty(UPropertyAnimatorCoreBase* InAnimator, FPropertyAnimatorCoreData& InProperty, bool bInTransact = false);
	PROPERTYANIMATORCORE_API bool UnlinkAnimatorProperties(UPropertyAnimatorCoreBase* InAnimator, const TSet<FPropertyAnimatorCoreData>& InProperties, bool bInTransact = false);
	PROPERTYANIMATORCORE_API bool UnlinkAnimatorProperties(const TSet<UPropertyAnimatorCoreContext*>& InPropertyContexts, bool bInTransact = false);

	/** Set the enabled state of animator property context */
	PROPERTYANIMATORCORE_API void SetAnimatorPropertiesEnabled(const TSet<UPropertyAnimatorCoreContext*>& InPropertyContexts, bool bInEnabled, bool bInTransact = false);

	/** Set the enabled state of animators attached to actors, will disable state globally on the component */
	PROPERTYANIMATORCORE_API void SetActorAnimatorsEnabled(const TSet<AActor*>& InActors, bool bInEnabled, bool bInTransact = false);

	/** Set the enabled state of animators in a world, will disable state globally on the component */
	PROPERTYANIMATORCORE_API void SetLevelAnimatorsEnabled(const UWorld* InWorld, bool bInEnabled, bool bInTransact = false);

	/** Set the enabled state of animators provided */
	PROPERTYANIMATORCORE_API void SetAnimatorsEnabled(const TSet<UPropertyAnimatorCoreBase*>& InAnimators, bool bInEnabled, bool bInTransact = false);

protected:
	/** Delegate to change state of animators in a world */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnAnimatorsSetEnabled, const UWorld* /** InAnimatorWorld */, bool /** bInAnimatorEnabled */, bool /** bInTransact */)
	static FOnAnimatorsSetEnabled OnAnimatorsSetEnabledDelegate;

	/**
	 * Scan for children of each of the following classes and registers their CDO:
	 * 1. UPropertyAnimatorCoreBase
	 * 2. UPropertyAnimatorCoreHandlerBase
	 * 3. UPropertyAnimatorCoreResolver
	 * 4. UPropertyAnimatorCoreTimeSourceBase
	 * 5. UPropertyAnimatorCorePresetBase
	 * 6. UPropertyAnimatorCoreConverterBase
	 */
	void RegisterAnimatorClasses();

	void OnAssetRegistryFilesLoaded();
	void OnAssetRegistryAssetAdded(const FAssetData& InAssetData);
	void OnAssetRegistryAssetRemoved(const FAssetData& InAssetData);
	void OnAssetRegistryAssetUpdated(const FAssetData& InAssetData);
	void RegisterPresetAsset(const FAssetData& InAssetData);
	void UnregisterPresetAsset(const FAssetData& InAssetData);

	/** Time sources available to use with animators */
	UPROPERTY()
	TSet<TWeakObjectPtr<UPropertyAnimatorCoreTimeSourceBase>> TimeSourcesWeak;

	/** Animators available to link properties to */
	UPROPERTY()
	TSet<TWeakObjectPtr<UPropertyAnimatorCoreBase>> AnimatorsWeak;

	/** Handlers are used to set/get same type properties and reuse logic */
	UPROPERTY()
	TSet<TWeakObjectPtr<UPropertyAnimatorCoreHandlerBase>> HandlersWeak;

	/** Resolvers find properties to let user control them when they are unreachable/hidden */
	UPROPERTY()
	TSet<TWeakObjectPtr<UPropertyAnimatorCoreResolver>> ResolversWeak;

	/** Presets available to apply on animator */
	UPROPERTY()
	TSet<TWeakObjectPtr<UPropertyAnimatorCorePresetBase>> PresetsWeak;

	/** Converters available to transform a type to another type */
	UPROPERTY()
	TSet<TWeakObjectPtr<UPropertyAnimatorCoreConverterBase>> ConvertersWeak;

	/** Some property and their setter cannot be identified automatically, use manual setter resolvers */
	TMap<FName, TFunction<UFunction*(const UObject*)>> SetterResolvers;

	/** Some property should have a friendlier name and replace the original name by an alias */
	TMap<FString, FString> PropertyAliases;

	bool bFilesLoaded = false;
};
