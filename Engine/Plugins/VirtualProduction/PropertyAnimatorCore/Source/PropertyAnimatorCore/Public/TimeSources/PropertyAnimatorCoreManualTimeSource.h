// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyAnimatorCoreTimeSourceBase.h"
#include "PropertyAnimatorCoreManualTimeSource.generated.h"

UENUM()
enum class EPropertyAnimatorCoreManualStatus : uint8
{
	/** Animation is done */
	Stopped,
	/** Animation is paused */
	Paused,
	/** Animation is playing */
	PlayingForward,
	/** Animation is playing in reverse */
	PlayingBackward
};

UCLASS(MinimalAPI)
class UPropertyAnimatorCoreManualTimeSource : public UPropertyAnimatorCoreTimeSourceBase
{
	GENERATED_BODY()

public:
	static PROPERTYANIMATORCORE_API FName GetCustomTimePropertyName();

	UPropertyAnimatorCoreManualTimeSource()
		: UPropertyAnimatorCoreTimeSourceBase(TEXT("Manual"))
	{}

	PROPERTYANIMATORCORE_API void SetCustomTime(double InTime);
	double GetCustomTime() const
	{
		return CustomTime;
	}

	PROPERTYANIMATORCORE_API void SetPlaybackState(EPropertyAnimatorCoreManualStatus InState);
	EPropertyAnimatorCoreManualStatus GetPlaybackState() const
	{
		return PlaybackState;
	}

	void Play(bool bInForward);
	void Pause();
	void Stop();

	EPropertyAnimatorCoreManualStatus GetPlaybackStatus() const;
	bool IsPlaying() const;

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UPropertyAnimatorTimeSourceBase
	virtual bool UpdateEvaluationData(FPropertyAnimatorCoreTimeSourceEvaluationData& OutData) override;
	virtual void OnTimeSourceActive() override;
	virtual void OnTimeSourceInactive() override;
	virtual bool ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue) override;
	virtual bool ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const override;
	//~ End UPropertyAnimatorTimeSourceBase

	void OnStateChanged();

	/** Time to evaluate */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator")
	double CustomTime = 0.f;

	/**
	 * Playback state
	 * Stopped = 0
	 * Paused = 1
	 * PlayingForward = 2
	 * PlayingReverse = 3
	 */
	UPROPERTY(EditInstanceOnly, Transient, DuplicateTransient, NoClear, Setter, Getter, Category="Animator")
	EPropertyAnimatorCoreManualStatus PlaybackState = EPropertyAnimatorCoreManualStatus::Paused;

	/** Current active status for the player */
	EPropertyAnimatorCoreManualStatus ActiveStatus = EPropertyAnimatorCoreManualStatus::Paused;
};