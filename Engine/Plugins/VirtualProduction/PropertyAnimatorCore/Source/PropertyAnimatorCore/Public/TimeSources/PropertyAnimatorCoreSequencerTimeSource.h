// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyAnimatorCoreTimeSourceBase.h"
#include "PropertyAnimatorCoreSequencerTimeSource.generated.h"

USTRUCT()
struct FPropertyAnimatorCoreSequencerTimeSourceEvalResult
{
	GENERATED_BODY()

	/** Is the evaluation state valid */
	bool bEvalValid = false;

	/** Last evaluated time received */
	UPROPERTY(VisibleInstanceOnly, Transient, Category="Animator", meta=(Units=Seconds))
	double EvalTime = 0.0;

	/** Last evaluated magnitude received */
	float EvalMagnitude = 1.f;
};

/** Time source that sync with a sequencer animator track */
UCLASS(MinimalAPI)
class UPropertyAnimatorCoreSequencerTimeSource : public UPropertyAnimatorCoreTimeSourceBase
{
	GENERATED_BODY()

public:
	UPropertyAnimatorCoreSequencerTimeSource()
		: UPropertyAnimatorCoreTimeSourceBase(TEXT("Sequencer"))
	{}

	//~ Begin UPropertyAnimatorTimeSourceBase
	virtual bool UpdateEvaluationData(FPropertyAnimatorCoreTimeSourceEvaluationData& OutData) override;
	//~ End UPropertyAnimatorTimeSourceBase

	void OnSequencerTimeEvaluated(const TOptional<double>& InTimeEval, const TOptional<float>& InMagnitudeEval);

protected:
	UPROPERTY(EditInstanceOnly, DisplayName="EvalTime", Category="Animator")
	FPropertyAnimatorCoreSequencerTimeSourceEvalResult EvalResult;
};