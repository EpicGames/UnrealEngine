// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/DateTime.h"
#include "Misc/Timespan.h"
#include "PropertyAnimatorCoreTimeSourceBase.h"
#include "PropertyAnimatorCoreSystemTimeSource.generated.h"

/** Enumerates all possible modes for the machine clock time source */
UENUM(BlueprintType)
enum class EPropertyAnimatorCoreSystemMode : uint8
{
	/** Local time of the machine */
	LocalTime,
	/** Specified duration elapsing until it reaches 0 */
	Countdown,
	/** Current time elapsed since the time source is active */
	Stopwatch
};

/** Enumerates all possible format interpretation in countdown mode */
UENUM(BlueprintType)
enum class EPropertyAnimatorCoreSystemCountdownFormat : uint8
{
	/**
	 * Default: format time provided is the duration of the countdown (relative)
	 * eg: if current time is 10:00:00 and format is set to 12:00:00,
	 * then countdown duration will be 12 hours until it reaches 0
	 */
	Duration,
	/**
	 * Format time provided is the target time (absolute)
	 * eg: if current time is 10:00.00 and format is set to 12:00:00,
	 * then countdown is 02:00:00 since 2 hours remains before reaching target time
	 */
	Target
};

/** System time source that support various option */
UCLASS(MinimalAPI)
class UPropertyAnimatorCoreSystemTimeSource : public UPropertyAnimatorCoreTimeSourceBase
{
	GENERATED_BODY()

public:
	UPropertyAnimatorCoreSystemTimeSource()
		: UPropertyAnimatorCoreTimeSourceBase(TEXT("System"))
	{}

	void SetMode(EPropertyAnimatorCoreSystemMode InMode);
	EPropertyAnimatorCoreSystemMode GetMode() const
	{
		return Mode;
	}

	void SetUseUtc(bool bInUseUtc);
	bool GetUseUtc() const
	{
		return bUseUtc;
	}

	void SetCountdownFormat(EPropertyAnimatorCoreSystemCountdownFormat InFormat);
	EPropertyAnimatorCoreSystemCountdownFormat GetCountdownFormat() const
	{
		return CountdownFormat;
	}

	void SetCountdownDuration(const FTimespan& InTimeSpan);
	void GetCountdownDuration(FTimespan& OutTimeSpan) const
	{
		OutTimeSpan = CountdownTimeSpan;
	}

	void SetCountdownDuration(const FString& InDuration);
	const FString& GetCountdownDuration() const
	{
		return CountdownDuration;
	}

protected:
	static FTimespan ParseTime(const FString& InFormat);

	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UPropertyAnimatorTimeSourceBase
	virtual bool UpdateEvaluationData(FPropertyAnimatorCoreTimeSourceEvaluationData& OutData) override;
	virtual void OnTimeSourceActive() override;
	virtual bool ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue) override;
	virtual bool ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const override;
	//~ End UPropertyAnimatorTimeSourceBase

	void OnModeChanged();
	void SetActivationTime();

	/** Machine time mode to use */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator")
	EPropertyAnimatorCoreSystemMode Mode = EPropertyAnimatorCoreSystemMode::LocalTime;

	/** Use universal time for calculations = greenwich mean time */
	UPROPERTY(EditInstanceOnly, Setter="SetUseUtc", Getter="GetUseUtc", Category="Animator")
	bool bUseUtc = false;

	/* Format interpretation for the countdown */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(EditCondition="Mode == EPropertyAnimatorCoreSystemMode::Countdown", EditConditionHides))
	EPropertyAnimatorCoreSystemCountdownFormat CountdownFormat = EPropertyAnimatorCoreSystemCountdownFormat::Duration;

	/**
	* Countdown time format (duration or target) : 
	* 120 = 2 minutes
	* 02:00 = 2 minutes
	* 00:02:00 = 2 minutes
	* 2m = 2 minutes
	* 1h = 1 hour
	* 120s = 2 minutes
	*/
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Countdown Time", Category="Animator", meta=(EditCondition="Mode == EPropertyAnimatorCoreSystemMode::Countdown", EditConditionHides))
	FString CountdownDuration = TEXT("23:59:59");

private:
	UPROPERTY(Transient)
	FTimespan CountdownTimeSpan;

	UPROPERTY(Transient)
	FDateTime ActivationTime;
};
