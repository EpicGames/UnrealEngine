// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Presets/PropertyAnimatorCorePresetable.h"
#include "PropertyAnimatorCoreTimeSourceBase.generated.h"

class UPropertyAnimatorCoreBase;

/** Enumerates all possible outcomes for the time source */
enum class EPropertyAnimatorCoreTimeSourceResult
{
	/** Time is the same as previous, evaluation can be skipped */
	Skip,
	/** Time is in an invalid state or out of range */
	Idle,
	/** Time is valid and in range, evaluate time */
	Evaluate
};

/** Stores all the data used by animators during evaluation */
struct FPropertyAnimatorCoreTimeSourceEvaluationData
{
	/** Time elapsed for animators evaluation */
	double TimeElapsed = 0.0;

	/** Time magnitude for animators to fade in/out effect based on time */
	float Magnitude = 1.f;
};

/**
 * Abstract base class for time source used by property animators
 * Can be transient or saved to disk if contains user set data
 */
UCLASS(MinimalAPI, Abstract)
class UPropertyAnimatorCoreTimeSourceBase : public UObject, public IPropertyAnimatorCorePresetable
{
	GENERATED_BODY()

	friend class UPropertyAnimatorCoreSubsystem;

public:
	UPropertyAnimatorCoreTimeSourceBase()
		: UPropertyAnimatorCoreTimeSourceBase(NAME_None)
	{}

	UPropertyAnimatorCoreTimeSourceBase(const FName& InSourceName)
		: TimeSourceName(InSourceName)
	{}

	void ActivateTimeSource();
	void DeactivateTimeSource();

	bool IsTimeSourceActive() const
	{
		return bTimeSourceActive;
	}

	EPropertyAnimatorCoreTimeSourceResult FetchEvaluationData(FPropertyAnimatorCoreTimeSourceEvaluationData& OutEvaluationData);

	FName GetTimeSourceName() const
	{
		return TimeSourceName;
	}

	void SetFrameRate(float InFrameRate);
	float GetFrameRate() const
	{
		return FrameRate;
	}

	void SetUseFrameRate(bool bInUseFrameRate);
	bool GetUseFrameRate() const
	{
		return bUseFrameRate;
	}

	double GetLastTimeElapsed() const
	{
		return LastTimeElapsed;
	}

	//~ Begin IPropertyAnimatorCorePresetable
	PROPERTYANIMATORCORE_API virtual bool ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue) override;
	PROPERTYANIMATORCORE_API virtual bool ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const override;
	//~ End IPropertyAnimatorCorePresetable

protected:
	/** Retrieve evaluation data to provide animators, return true if data is valid, false otherwise */
	PROPERTYANIMATORCORE_API virtual bool UpdateEvaluationData(FPropertyAnimatorCoreTimeSourceEvaluationData& OutData);

	/** Time source CDO is registered by subsystem */
	virtual void OnTimeSourceRegistered() {}

	/** Time source CDO is unregistered by subsystem */
	virtual void OnTimeSourceUnregistered() {}

	/** Time source is active on the animator */
	virtual void OnTimeSourceActive() {}

	/** Time source is inactive on the animator */
	virtual void OnTimeSourceInactive() {}

private:
	bool IsFramerateAllowed(double InNewTime) const;

	/** Use a specific framerate */
	UPROPERTY(EditInstanceOnly, Setter="SetUseFrameRate", Getter="GetUseFrameRate", Category="Animator", meta=(InlineEditConditionToggle))
	bool bUseFrameRate = false;

	/** The frame rate to target for the animator effect */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(ClampMin="0", EditCondition="bUseFrameRate"))
	float FrameRate = 30.f;

	/** Name used to display this time source to the user */
	UPROPERTY(Transient)
	FName TimeSourceName;

	/** Cached time elapsed */
	double LastTimeElapsed = 0;

	/** Is this time source active on the animator */
	bool bTimeSourceActive = false;
};