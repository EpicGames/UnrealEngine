// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyAnimatorCoreTimeSourceBase.h"
#include "PropertyAnimatorCoreWorldTimeSource.generated.h"

/**
 * Time source that follows world time,
 * Transient because it does not contain user saved data, can be recreated
 */
UCLASS(MinimalAPI, Transient)
class UPropertyAnimatorCoreWorldTimeSource : public UPropertyAnimatorCoreTimeSourceBase
{
	GENERATED_BODY()

public:
	UPropertyAnimatorCoreWorldTimeSource()
		: UPropertyAnimatorCoreTimeSourceBase(TEXT("World"))
	{}

	//~ Begin UPropertyAnimatorTimeSourceBase
	virtual bool UpdateEvaluationData(FPropertyAnimatorCoreTimeSourceEvaluationData& OutData) override;
	//~ End UPropertyAnimatorTimeSourceBase
};