// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/PropertyAnimatorCoreEditorContextTypeCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "Subsystems/PropertyAnimatorCoreSubsystem.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorCoreEditorContextTypeCustomization"

void FPropertyAnimatorCoreEditorContextTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InRow, IPropertyTypeCustomizationUtils& InUtils)
{
	if (!InPropertyHandle->IsValidHandle())
	{
		return;
	}

	// Dive into the array handle
	const TSharedPtr<IPropertyHandle> LinkedPropertiesHandle = InPropertyHandle->GetChildHandle(0);
	if (!LinkedPropertiesHandle.IsValid() || !LinkedPropertiesHandle->IsValidHandle())
	{
		return;
	}

	// Dive into array slot
	const TSharedPtr<IPropertyHandle> LinkedPropertyHandle = LinkedPropertiesHandle->GetChildHandle(0);
	if (!LinkedPropertyHandle.IsValid() || !LinkedPropertyHandle->IsValidHandle())
	{
		return;
	}

	TArray<UObject*> Objects;
	LinkedPropertyHandle->GetOuterObjects(Objects);
	if (Objects.IsEmpty())
	{
		return;
	}

	const UPropertyAnimatorCoreContext* PropertyContext = Cast<UPropertyAnimatorCoreContext>(Objects[0]);
	if (!PropertyContext)
	{
		return;
	}

	PropertyContextHandle = LinkedPropertyHandle;
	const FString PropertyDisplayName = PropertyContext->GetAnimatedProperty().GetPropertyDisplayName();
	const FName PropertyTypeName = PropertyContext->GetAnimatedProperty().GetLeafPropertyTypeName();

	InRow
		.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FPropertyAnimatorCoreEditorContextTypeCustomization::IsPropertyEnabled)
				.OnCheckStateChanged(this, &FPropertyAnimatorCoreEditorContextTypeCustomization::OnPropertyEnabled)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				LinkedPropertiesHandle->CreatePropertyNameWidget(FText::FromString(PropertyDisplayName + TEXT(" (") + PropertyTypeName.ToString() + TEXT(")")))
			]
		]
		.ValueContent()
		.HAlign(HAlign_Left)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f, 0.f)
			[
				SNew(SButton)
				.ContentPadding(2.f)
				.ToolTipText(LOCTEXT("UnlinkProperty", "Unlink property from animator"))
				.OnClicked(this, &FPropertyAnimatorCoreEditorContextTypeCustomization::UnlinkProperty)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
					.DesiredSizeOverride(FVector2D(16.f))
				]
			]
		];
}

void FPropertyAnimatorCoreEditorContextTypeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InBuilder, IPropertyTypeCustomizationUtils& InUtils)
{
	static const TSet<FName> SkipProperties
	{
		GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCoreContext, bAnimated)
	};

	if (PropertyContextHandle.IsValid() && PropertyContextHandle->IsValidHandle())
	{
		uint32 ChildrenCount = 0;
		PropertyContextHandle->GetNumChildren(ChildrenCount);

		for (uint32 ChildIndex = 0; ChildIndex < ChildrenCount; ++ChildIndex)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = PropertyContextHandle->GetChildHandle(ChildIndex);

			if (ChildHandle.IsValid() && !SkipProperties.Contains(ChildHandle->GetProperty()->GetFName()))
			{
				InBuilder.AddProperty(ChildHandle.ToSharedRef());
			}
		}
	}
}

ECheckBoxState FPropertyAnimatorCoreEditorContextTypeCustomization::IsPropertyEnabled() const
{
	TOptional<ECheckBoxState> State;

	if (PropertyContextHandle.IsValid())
	{
		TArray<UObject*> Objects;
		PropertyContextHandle->GetOuterObjects(Objects);

		for (UObject* Object : Objects)
		{
			if (UPropertyAnimatorCoreContext* ContextObject = Cast<UPropertyAnimatorCoreContext>(Object))
			{
				const ECheckBoxState ContextState = ContextObject->IsAnimated() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;

				if (!State.IsSet())
				{
					State = ContextState;
				}
				else if (State.GetValue() != ContextState)
				{
					State = ECheckBoxState::Undetermined;
					break;
				}
			}
		}
	}

	return State.Get(ECheckBoxState::Undetermined);
}

void FPropertyAnimatorCoreEditorContextTypeCustomization::OnPropertyEnabled(ECheckBoxState InNewState) const
{
	if (!PropertyContextHandle.IsValid())
	{
		return;
	}

	TArray<UObject*> Objects;
	PropertyContextHandle->GetOuterObjects(Objects);

	TSet<UPropertyAnimatorCoreContext*> PropertyContexts;
	Algo::Transform(Objects, PropertyContexts, [](UObject* InObject)
	{
		return Cast<UPropertyAnimatorCoreContext>(InObject);
	});

	if (UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get())
	{
		constexpr bool bShouldTransact = true;
		AnimatorSubsystem->SetAnimatorPropertiesEnabled(PropertyContexts, InNewState == ECheckBoxState::Checked, bShouldTransact);
	}
}

FReply FPropertyAnimatorCoreEditorContextTypeCustomization::UnlinkProperty() const
{
	if (!PropertyContextHandle.IsValid())
	{
		return FReply::Unhandled();
	}

	TArray<UObject*> Objects;
	PropertyContextHandle->GetOuterObjects(Objects);

	TSet<UPropertyAnimatorCoreContext*> PropertyContexts;
	Algo::Transform(Objects, PropertyContexts, [](UObject* InObject)
	{
		return Cast<UPropertyAnimatorCoreContext>(InObject);
	});

	if (UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get())
	{
		constexpr bool bShouldTransact = true;
		AnimatorSubsystem->UnlinkAnimatorProperties(PropertyContexts, bShouldTransact);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
