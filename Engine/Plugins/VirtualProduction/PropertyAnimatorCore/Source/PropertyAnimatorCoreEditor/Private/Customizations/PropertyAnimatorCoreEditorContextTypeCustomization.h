// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class FReply;
class IDetailPropertyRow;
class IPropertyUtilities;
class UPropertyAnimatorCoreContext;
struct FPropertyAnimatorCoreData;
enum class ECheckBoxState : uint8;

/** Type customization for UPropertyAnimatorCoreContext */
class FPropertyAnimatorCoreEditorContextTypeCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FPropertyAnimatorCoreEditorContextTypeCustomization>();
	}

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InRow, IPropertyTypeCustomizationUtils& InUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InBuilder, IPropertyTypeCustomizationUtils& InUtils) override;
	//~ End IPropertyTypeCustomization

protected:
	ECheckBoxState IsPropertyEnabled() const;
	void OnPropertyEnabled(ECheckBoxState InNewState) const;
	FReply UnlinkProperty() const;

	TSharedPtr<IPropertyHandle> PropertyContextHandle;
};
