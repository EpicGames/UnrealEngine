// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/PropertyAnimatorCoreEditorDetailCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Menus/PropertyAnimatorCoreEditorMenuContext.h"
#include "Presets/PropertyAnimatorCorePropertyPreset.h"
#include "Styles/PropertyAnimatorCoreEditorStyle.h"
#include "Subsystems/PropertyAnimatorCoreEditorSubsystem.h"
#include "Subsystems/PropertyAnimatorCoreSubsystem.h"
#include "ToolMenus.h"
#include "ToolMenu.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorCoreEditorDetailCustomization"

void FPropertyAnimatorCoreEditorDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	const TSharedPtr<IPropertyHandle> LinkedPropertiesHandle = InDetailBuilder.GetProperty(
		UPropertyAnimatorCoreBase::GetLinkedPropertiesPropertyName(),
		UPropertyAnimatorCoreBase::StaticClass()
	);

	if (!LinkedPropertiesHandle.IsValid())
	{
		return;
	}

	AnimatorsWeak = InDetailBuilder.GetObjectsOfTypeBeingCustomized<UPropertyAnimatorCoreBase>();

	if (AnimatorsWeak.IsEmpty())
	{
		return;
	}

	IDetailPropertyRow* PropertyRow = InDetailBuilder.EditDefaultProperty(LinkedPropertiesHandle);

	if (!PropertyRow)
	{
		return;
	}

	PropertyRow->ShouldAutoExpand(true);

	const TSharedRef<SWidget> NewValueWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.f, 0.f)
		[
			SNew(SButton)
			.ContentPadding(2.f)
			.ToolTipText(LOCTEXT("UnlinkProperties", "Unlink properties from this animator"))
			.IsEnabled(this, &FPropertyAnimatorCoreEditorDetailCustomization::IsAnyPropertyLinked)
			.OnClicked(this, &FPropertyAnimatorCoreEditorDetailCustomization::UnlinkProperties)
			.Content()
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(16.f))
				.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.f, 0.f)
		[
			SNew(SComboButton)
			.ContentPadding(2.f)
			.ButtonStyle(&FCoreStyle::Get().GetWidgetStyle<FButtonStyle>("Button"))
			.ToolTipText(LOCTEXT("LinkProperties", "Link properties to this animator"))
			.HasDownArrow(false)
			.OnGetMenuContent(this, &FPropertyAnimatorCoreEditorDetailCustomization::GenerateLinkMenu)
			.ButtonContent()
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(16.f))
				.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.f, 0.f)
		[
			SNew(SButton)
			.ContentPadding(2.f)
			.ButtonStyle(&FCoreStyle::Get().GetWidgetStyle<FButtonStyle>("Button"))
			.ToolTipText(LOCTEXT("CreatePropertyPreset", "Create a preset from these properties"))
			.Visibility(AnimatorsWeak.Num() == 1 ? EVisibility::Visible : EVisibility::Collapsed)
			.OnClicked(this, &FPropertyAnimatorCoreEditorDetailCustomization::OnCreatePropertyPresetClicked)
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(16.f))
				.Image(FPropertyAnimatorCoreEditorStyle::Get().GetBrush("PropertyControlIcon.Export"))
			]
		];

	if (FDetailWidgetDecl* ValueWidget = PropertyRow->CustomValueWidget())
	{
		ValueWidget->Widget = NewValueWidget;
	}
	else
	{
		PropertyRow->CustomWidget(true)
		.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsEnabled(this, &FPropertyAnimatorCoreEditorDetailCustomization::IsAnyPropertyLinked)
				.IsChecked(this, &FPropertyAnimatorCoreEditorDetailCustomization::IsPropertiesEnabled)
				.OnCheckStateChanged(this, &FPropertyAnimatorCoreEditorDetailCustomization::OnPropertiesEnabled)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				LinkedPropertiesHandle->CreatePropertyNameWidget()
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Font(InDetailBuilder.GetDetailFont())
				.Text(this, &FPropertyAnimatorCoreEditorDetailCustomization::GetPropertiesCountText)
			]
		]
		.ValueContent()
		[
			NewValueWidget
		];
	}
}

int32 FPropertyAnimatorCoreEditorDetailCustomization::GetPropertiesCount() const
{
	int32 Count = INDEX_NONE;
	for (const TWeakObjectPtr<UPropertyAnimatorCoreBase>& AnimatorWeak : AnimatorsWeak)
	{
		const UPropertyAnimatorCoreBase* Animator = AnimatorWeak.Get();
		if (!Animator)
		{
			continue;
		}

		const int32 LinkedPropertiesCount = Animator->GetLinkedPropertiesCount();

		if (Count == INDEX_NONE)
		{
			Count = LinkedPropertiesCount;
		}
		else if (Count != LinkedPropertiesCount)
		{
			Count = INDEX_NONE;
			break;
		}
	}

	return Count;
}

FText FPropertyAnimatorCoreEditorDetailCustomization::GetPropertiesCountText() const
{
	const int32 Count = GetPropertiesCount();

	if (Count != INDEX_NONE)
	{
		return FText::Format(LOCTEXT("LinkedPropertiesCount", " ({0}) "), FText::FromString(FString::FromInt(Count)));
	}

	return LOCTEXT("LinkedPropertiesCountMismatch", " (multiple) ");
}

TSharedRef<SWidget> FPropertyAnimatorCoreEditorDetailCustomization::GenerateLinkMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();

	if (AnimatorsWeak.IsEmpty() || !ToolMenus)
	{
		return SNullWidget::NullWidget;
	}

	static constexpr TCHAR MenuName[] = TEXT("LinkPropertiesCustomizationMenu");

	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* const LinkPropertiesMenu = ToolMenus->RegisterMenu(MenuName, NAME_None, EMultiBoxType::Menu);
		LinkPropertiesMenu->AddDynamicSection(TEXT("FillLinkPropertiesCustomizationMenu"), FNewToolMenuDelegate::CreateStatic(&FPropertyAnimatorCoreEditorDetailCustomization::FillLinkMenu));
	}

	UPropertyAnimatorCoreEditorMenuContext* MenuContext = NewObject<UPropertyAnimatorCoreEditorMenuContext>();

	TSet<UPropertyAnimatorCoreBase*> Animators;
	Algo::Transform(AnimatorsWeak, Animators, [](const TWeakObjectPtr<UPropertyAnimatorCoreBase>& InAnimatorWeak)
	{
		return InAnimatorWeak.Get();
	});
	MenuContext->SetAnimators(Animators);
	return ToolMenus->GenerateWidget(MenuName, FToolMenuContext(MenuContext));
}

void FPropertyAnimatorCoreEditorDetailCustomization::FillLinkMenu(UToolMenu* InToolMenu)
{
	UPropertyAnimatorCoreEditorSubsystem* AnimatorEditorSubsystem = UPropertyAnimatorCoreEditorSubsystem::Get();
	const UPropertyAnimatorCoreEditorMenuContext* MenuContextObject = InToolMenu->FindContext<UPropertyAnimatorCoreEditorMenuContext>();

	if (!AnimatorEditorSubsystem || !MenuContextObject)
	{
		return;
	}

	TSet<UObject*> ContextObjects;
	Algo::TransformIf(
		MenuContextObject->GetAnimators(),
		ContextObjects,
		[](const UPropertyAnimatorCoreBase* InAnimator)
		{
			return IsValid(InAnimator);
		},
		[](UPropertyAnimatorCoreBase* InAnimator)
		{
			return InAnimator;
		}
	);

	if (ContextObjects.IsEmpty())
	{
		return;
	}

	const FPropertyAnimatorCoreEditorMenuContext MenuContext(ContextObjects, {});
	FPropertyAnimatorCoreEditorMenuOptions MenuOptions(
		{
			EPropertyAnimatorCoreEditorMenuType::Link
		}
	);
	MenuOptions.CreateSubMenu(false);

	AnimatorEditorSubsystem->FillAnimatorMenu(InToolMenu, MenuContext, MenuOptions);
}

bool FPropertyAnimatorCoreEditorDetailCustomization::IsAnyPropertyLinked() const
{
	for (const TWeakObjectPtr<UPropertyAnimatorCoreBase>& AnimatorWeak : AnimatorsWeak)
	{
		const UPropertyAnimatorCoreBase* Animator = AnimatorWeak.Get();
		if (!Animator)
		{
			continue;
		}

		if (Animator->GetLinkedPropertiesCount() > 0)
		{
			return true;
		}
	}

	return false;
}

ECheckBoxState FPropertyAnimatorCoreEditorDetailCustomization::IsPropertiesEnabled() const
{
	TOptional<ECheckBoxState> State;
	for (const TWeakObjectPtr<UPropertyAnimatorCoreBase>& AnimatorWeak : AnimatorsWeak)
	{
		UPropertyAnimatorCoreBase* Animator = AnimatorWeak.Get();
		if (!Animator)
		{
			continue;
		}

		int32 EnabledProperties = 0;
		constexpr bool bResolve = false;
		Animator->ForEachLinkedProperty<UPropertyAnimatorCoreContext>([&EnabledProperties](UPropertyAnimatorCoreContext* InOptions, const FPropertyAnimatorCoreData& InProperty)->bool
		{
			if (InOptions->IsAnimated())
			{
				EnabledProperties++;
			}
			return true;
		}, bResolve);
		
		const int32 LinkedProperties = Animator->GetLinkedPropertiesCount();

		ECheckBoxState AnimatorState = ECheckBoxState::Undetermined;
		if (LinkedProperties > 0 && EnabledProperties == 0)
		{
			AnimatorState = ECheckBoxState::Unchecked;
		}
		else if (LinkedProperties > 0 && EnabledProperties < LinkedProperties)
		{
			AnimatorState = ECheckBoxState::Undetermined;
		}
		else
		{
			AnimatorState = ECheckBoxState::Checked;
		}

		if (!State.IsSet())
		{
			State = AnimatorState;
		}
		else if (State.GetValue() != AnimatorState)
		{
			State = ECheckBoxState::Undetermined;
			break;
		}
	}

	return State.Get(ECheckBoxState::Undetermined);
}

void FPropertyAnimatorCoreEditorDetailCustomization::OnPropertiesEnabled(ECheckBoxState InNewState) const
{
	if (InNewState == ECheckBoxState::Undetermined)
	{
		return;
	}

	TSet<UPropertyAnimatorCoreContext*> PropertyContexts;
	for (const TWeakObjectPtr<UPropertyAnimatorCoreBase>& AnimatorWeak : AnimatorsWeak)
	{
		const UPropertyAnimatorCoreBase* Animator = AnimatorWeak.Get();
		if (!Animator)
		{
			continue;
		}

		constexpr bool bResolve = false;
		Animator->ForEachLinkedProperty<UPropertyAnimatorCoreContext>([&PropertyContexts](UPropertyAnimatorCoreContext* InOptions, const FPropertyAnimatorCoreData& InProperty)->bool
		{
			PropertyContexts.Add(InOptions);
			return true;
		}, bResolve);
	}

	if (UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get())
	{
		constexpr bool bShouldTransact = true;
		AnimatorSubsystem->SetAnimatorPropertiesEnabled(PropertyContexts, InNewState == ECheckBoxState::Checked, bShouldTransact);
	}
}

FReply FPropertyAnimatorCoreEditorDetailCustomization::UnlinkProperties() const
{
	if (UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get())
	{
		TSet<UPropertyAnimatorCoreContext*> PropertyContexts;
		for (const TWeakObjectPtr<UPropertyAnimatorCoreBase>& AnimatorWeak : AnimatorsWeak)
		{
			const UPropertyAnimatorCoreBase* Animator = AnimatorWeak.Get();	
			if (!Animator)
			{
				continue;
			}

			PropertyContexts.Append(Animator->GetLinkedPropertiesContext());
		}

		constexpr bool bShouldTransact = true;
		AnimatorSubsystem->UnlinkAnimatorProperties(PropertyContexts, bShouldTransact);
	}

	return FReply::Handled();
}

FReply FPropertyAnimatorCoreEditorDetailCustomization::OnCreatePropertyPresetClicked() const
{
	if (AnimatorsWeak.IsEmpty())
	{
		return FReply::Handled();
	}

	const UPropertyAnimatorCoreBase* Animator = AnimatorsWeak[0].Get();
	if (!Animator)
	{
		return FReply::Handled();
	}

	if (UPropertyAnimatorCoreEditorSubsystem* AnimatorEditorSubsystem = UPropertyAnimatorCoreEditorSubsystem::Get())
	{
		AnimatorEditorSubsystem->CreatePresetAsset(UPropertyAnimatorCorePropertyPreset::StaticClass(), TArray<IPropertyAnimatorCorePresetable*>{Animator->GetLinkedPropertiesContext()});
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
