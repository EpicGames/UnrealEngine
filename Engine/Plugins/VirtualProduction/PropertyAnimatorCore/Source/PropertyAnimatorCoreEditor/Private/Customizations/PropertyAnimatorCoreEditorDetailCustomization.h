// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class FReply;
class SWidget;
class UPropertyAnimatorCoreBase;
class UToolMenu;
enum class ECheckBoxState : uint8;

/** Details customization for UPropertyAnimatorCoreBase */
class FPropertyAnimatorCoreEditorDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FPropertyAnimatorCoreEditorDetailCustomization>();
	}

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization

protected:
	static void FillLinkMenu(UToolMenu* InToolMenu);

	TSharedRef<SWidget> GenerateLinkMenu();

	bool IsAnyPropertyLinked() const;
	ECheckBoxState IsPropertiesEnabled() const;
	void OnPropertiesEnabled(ECheckBoxState InNewState) const;

	int32 GetPropertiesCount() const;
	FText GetPropertiesCountText() const;

	FReply UnlinkProperties() const;

	FReply OnCreatePropertyPresetClicked() const;

	TArray<TWeakObjectPtr<UPropertyAnimatorCoreBase>> AnimatorsWeak;
};
