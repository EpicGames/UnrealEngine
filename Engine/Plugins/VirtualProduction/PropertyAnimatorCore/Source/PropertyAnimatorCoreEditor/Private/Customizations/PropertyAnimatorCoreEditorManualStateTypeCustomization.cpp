// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/PropertyAnimatorCoreEditorManualStateTypeCustomization.h"

#include "DetailWidgetRow.h"
#include "Styles/PropertyAnimatorCoreEditorStyle.h"
#include "Styling/StyleColors.h"
#include "TimeSources/PropertyAnimatorCoreManualTimeSource.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"

void FPropertyAnimatorCoreEditorManualStateTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InRow, IPropertyTypeCustomizationUtils& InUtils)
{
	if (!InPropertyHandle->IsValidHandle())
	{
		return;
	}

	StatusPropertyHandle = InPropertyHandle;

	if (const TSharedPtr<IPropertyHandle> ParentHandle = InPropertyHandle->GetParentHandle())
	{
		CustomTimePropertyHandle = ParentHandle->GetChildHandle(UPropertyAnimatorCoreManualTimeSource::GetCustomTimePropertyName(), /** Recurse */false);
	}

	InRow.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	];

	const FVector2D ImageSize(16.f);

	InRow.ValueContent()
	.HAlign(HAlign_Fill)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.f, 0.f, 0.f, 5.f)
		.HAlign(HAlign_Fill)
		[
			SNew(SButton)
			.HAlign(HAlign_Fill)
			.ButtonColorAndOpacity(this, &FPropertyAnimatorCoreEditorManualStateTypeCustomization::IsPlaybackStatusActive, EPropertyAnimatorCoreManualStatus::PlayingBackward)
			.OnClicked(this, &FPropertyAnimatorCoreEditorManualStateTypeCustomization::SetPlaybackStatus, EPropertyAnimatorCoreManualStatus::PlayingBackward)
			.IsEnabled(this, &FPropertyAnimatorCoreEditorManualStateTypeCustomization::IsPlaybackStatusAllowed, EPropertyAnimatorCoreManualStatus::PlayingBackward)
			[
				SNew(SImage)
				.DesiredSizeOverride(ImageSize)
				.Image(FPropertyAnimatorCoreEditorStyle::Get().GetBrush(TEXT("ManualTimeSourceControl.PlayBackward")))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.f, 0.f, 0.f, 5.f)
		.HAlign(HAlign_Fill)
		[
			SNew(SButton)
			.HAlign(HAlign_Fill)
			.ButtonColorAndOpacity(this, &FPropertyAnimatorCoreEditorManualStateTypeCustomization::IsPlaybackStatusActive, EPropertyAnimatorCoreManualStatus::PlayingForward)
			.OnClicked(this, &FPropertyAnimatorCoreEditorManualStateTypeCustomization::SetPlaybackStatus, EPropertyAnimatorCoreManualStatus::PlayingForward)
			.IsEnabled(this, &FPropertyAnimatorCoreEditorManualStateTypeCustomization::IsPlaybackStatusAllowed, EPropertyAnimatorCoreManualStatus::PlayingForward)
			[
				SNew(SImage)
				.DesiredSizeOverride(ImageSize)
				.Image(FPropertyAnimatorCoreEditorStyle::Get().GetBrush(TEXT("ManualTimeSourceControl.PlayForward")))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.f, 0.f, 0.f, 5.f)
		.HAlign(HAlign_Fill)
		[
			SNew(SButton)
			.HAlign(HAlign_Fill)
			.ButtonColorAndOpacity(this, &FPropertyAnimatorCoreEditorManualStateTypeCustomization::IsPlaybackStatusActive, EPropertyAnimatorCoreManualStatus::Paused)
			.OnClicked(this, &FPropertyAnimatorCoreEditorManualStateTypeCustomization::SetPlaybackStatus, EPropertyAnimatorCoreManualStatus::Paused)
			.IsEnabled(this, &FPropertyAnimatorCoreEditorManualStateTypeCustomization::IsPlaybackStatusAllowed, EPropertyAnimatorCoreManualStatus::Paused)
			[
				SNew(SImage)
				.DesiredSizeOverride(ImageSize)
				.Image(FPropertyAnimatorCoreEditorStyle::Get().GetBrush(TEXT("ManualTimeSourceControl.Pause")))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.f, 0.f, 0.f, 5.f)
		.HAlign(HAlign_Fill)
		[
			SNew(SButton)
			.HAlign(HAlign_Fill)
			.ButtonColorAndOpacity(this, &FPropertyAnimatorCoreEditorManualStateTypeCustomization::IsPlaybackStatusActive, EPropertyAnimatorCoreManualStatus::Stopped)
			.OnClicked(this, &FPropertyAnimatorCoreEditorManualStateTypeCustomization::SetPlaybackStatus, EPropertyAnimatorCoreManualStatus::Stopped)
			.IsEnabled(this, &FPropertyAnimatorCoreEditorManualStateTypeCustomization::IsPlaybackStatusAllowed, EPropertyAnimatorCoreManualStatus::Stopped)
			[
				SNew(SImage)
				.DesiredSizeOverride(ImageSize)
				.Image(FPropertyAnimatorCoreEditorStyle::Get().GetBrush(TEXT("ManualTimeSourceControl.Stop")))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.f, 0.f, 0.f, 5.f)
		.HAlign(HAlign_Fill)
		[
			SNew(SButton)
			.HAlign(HAlign_Fill)
			.ButtonColorAndOpacity(FStyleColors::White)
			.OnClicked(this, &FPropertyAnimatorCoreEditorManualStateTypeCustomization::ResetPlaybackStatus)
			.Visibility(CustomTimePropertyHandle.IsValid() ? EVisibility::Visible : EVisibility::Collapsed)
			[
				SNew(SImage)
				.DesiredSizeOverride(ImageSize)
				.Image(FPropertyAnimatorCoreEditorStyle::Get().GetBrush(TEXT("ManualTimeSourceControl.Reset")))
			]
		]
	];
}

void FPropertyAnimatorCoreEditorManualStateTypeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InBuilder, IPropertyTypeCustomizationUtils& InUtils)
{
}

FReply FPropertyAnimatorCoreEditorManualStateTypeCustomization::SetPlaybackStatus(EPropertyAnimatorCoreManualStatus InStatus)
{
	if (StatusPropertyHandle.IsValid() && StatusPropertyHandle->IsValidHandle())
	{
		StatusPropertyHandle->SetValue(static_cast<uint8>(InStatus));
	}

	return FReply::Handled();
}

FReply FPropertyAnimatorCoreEditorManualStateTypeCustomization::ResetPlaybackStatus()
{
	if (StatusPropertyHandle.IsValid() && StatusPropertyHandle->IsValidHandle())
	{
		StatusPropertyHandle->SetValue(static_cast<uint8>(EPropertyAnimatorCoreManualStatus::Paused));
	}

	if (CustomTimePropertyHandle.IsValid() && CustomTimePropertyHandle->IsValidHandle())
	{
		CustomTimePropertyHandle->SetValue(0.0);
	}

	return FReply::Handled();
}

bool FPropertyAnimatorCoreEditorManualStateTypeCustomization::IsPlaybackStatusAllowed(EPropertyAnimatorCoreManualStatus InStatus) const
{
	if (StatusPropertyHandle.IsValid() && StatusPropertyHandle->IsValidHandle())
	{
		uint8 CurrentStatus = 0;
		if (StatusPropertyHandle->GetValue(CurrentStatus) == FPropertyAccess::Success)
		{
			switch(static_cast<EPropertyAnimatorCoreManualStatus>(CurrentStatus))
			{
			case EPropertyAnimatorCoreManualStatus::Stopped:
				{
					return InStatus == EPropertyAnimatorCoreManualStatus::PlayingForward
						|| InStatus == EPropertyAnimatorCoreManualStatus::PlayingBackward;
				}
			case EPropertyAnimatorCoreManualStatus::Paused:
				{
					return InStatus == EPropertyAnimatorCoreManualStatus::PlayingForward
						|| InStatus == EPropertyAnimatorCoreManualStatus::PlayingBackward
						|| InStatus == EPropertyAnimatorCoreManualStatus::Stopped;
				}
			case EPropertyAnimatorCoreManualStatus::PlayingForward:
				{
					return InStatus == EPropertyAnimatorCoreManualStatus::PlayingBackward
						|| InStatus == EPropertyAnimatorCoreManualStatus::Paused
						|| InStatus == EPropertyAnimatorCoreManualStatus::Stopped;
				}
			case EPropertyAnimatorCoreManualStatus::PlayingBackward:
				{
					return InStatus == EPropertyAnimatorCoreManualStatus::PlayingForward
						|| InStatus == EPropertyAnimatorCoreManualStatus::Paused
						|| InStatus == EPropertyAnimatorCoreManualStatus::Stopped;
				}
			}
		}
	}

	return false;
}

FSlateColor FPropertyAnimatorCoreEditorManualStateTypeCustomization::IsPlaybackStatusActive(EPropertyAnimatorCoreManualStatus InStatus) const
{
	if (StatusPropertyHandle.IsValid() && StatusPropertyHandle->IsValidHandle())
	{
		uint8 CurrentStatus = 0;
		if (StatusPropertyHandle->GetValue(CurrentStatus) == FPropertyAccess::Success)
		{
			if (InStatus == static_cast<EPropertyAnimatorCoreManualStatus>(CurrentStatus))
			{
				return FStyleColors::Select;
			}
		}
	}

	return FStyleColors::White;
}
