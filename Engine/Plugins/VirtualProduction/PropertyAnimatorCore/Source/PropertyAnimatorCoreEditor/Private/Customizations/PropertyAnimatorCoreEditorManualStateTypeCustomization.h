// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class FReply;
class IPropertyHandle;
enum class EPropertyAnimatorCoreManualStatus : uint8;
struct FSlateColor;

/** Type customization for EPropertyAnimatorCoreManualStatus to show a player */
class FPropertyAnimatorCoreEditorManualStateTypeCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FPropertyAnimatorCoreEditorManualStateTypeCustomization>();
	}

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InRow, IPropertyTypeCustomizationUtils& InUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InBuilder, IPropertyTypeCustomizationUtils& InUtils) override;
	//~ End IPropertyTypeCustomization

protected:
	FReply SetPlaybackStatus(EPropertyAnimatorCoreManualStatus InStatus);
	FReply ResetPlaybackStatus();
	bool IsPlaybackStatusAllowed(EPropertyAnimatorCoreManualStatus InStatus) const;
	FSlateColor IsPlaybackStatusActive(EPropertyAnimatorCoreManualStatus InStatus) const;

	TSharedPtr<IPropertyHandle> StatusPropertyHandle;
	TSharedPtr<IPropertyHandle> CustomTimePropertyHandle;
};
