// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/PropertyAnimatorCoreEditorSeedTypeCustomization.h"

#include "DetailWidgetRow.h"
#include "Input/Reply.h"
#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorCoreEditorSeedTypeCustomization"

void FPropertyAnimatorCoreEditorSeedTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InRow, IPropertyTypeCustomizationUtils& InUtils)
{
	check(InPropertyHandle->IsValidHandle() && !!InPropertyHandle->GetProperty() && InPropertyHandle->GetProperty()->IsA<FNumericProperty>())

	InRow.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	];

	InRow.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.Padding(0.f, 0.f, 3.f, 0.f)
		[
			InPropertyHandle->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.OnClicked_Static(&FPropertyAnimatorCoreEditorSeedTypeCustomization::OnGenerateSeedClicked, InPropertyHandle.ToWeakPtr())
			[
				SNew(STextBlock)
				.Font(InUtils.GetRegularFont())
				.Text(LOCTEXT("GenerateSeed", "Seed"))
				.ToolTipText(LOCTEXT("GenerateSeedTooltip", "Generates a new seed between property type min and max value"))
			]
		]
	];
}

void FPropertyAnimatorCoreEditorSeedTypeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InBuilder, IPropertyTypeCustomizationUtils& InUtils)
{
}

FReply FPropertyAnimatorCoreEditorSeedTypeCustomization::OnGenerateSeedClicked(TWeakPtr<IPropertyHandle> InPropertyHandleWeak)
{
	if (const TSharedPtr<IPropertyHandle> PropertyHandle = InPropertyHandleWeak.Pin())
	{
		const FNumericProperty* NumericProperty = CastField<FNumericProperty>(PropertyHandle->GetProperty());

		if (NumericProperty && !NumericProperty->IsEnum())
		{
			if (NumericProperty->IsInteger())
			{
				int64 MinValue = 0;
				int64 MaxValue = 0;

				if (NumericProperty->IsA<FInt64Property>() || NumericProperty->IsA<FIntProperty>())
				{
					MinValue = TNumericLimits<int32>::Min();
					MaxValue = TNumericLimits<int32>::Max();
				}
				else if (NumericProperty->IsA<FByteProperty>())
				{
					MinValue = TNumericLimits<uint8>::Min();
					MaxValue = TNumericLimits<uint8>::Max();
				}

				if (PropertyHandle->HasMetaData(TEXT("ClampMin")))
				{
					MinValue = FMath::Max(PropertyHandle->GetIntMetaData(TEXT("ClampMin")), MinValue);
				}

				if (PropertyHandle->HasMetaData(TEXT("ClampMax")))
				{
					MaxValue = FMath::Min(PropertyHandle->GetIntMetaData(TEXT("ClampMax")), MaxValue);
				}

				for (int32 Index = 0; Index < PropertyHandle->GetNumPerObjectValues(); Index++)
				{
					const int32 Value = FMath::Lerp(MinValue, MaxValue, FMath::FRand());
					PropertyHandle->SetPerObjectValue(Index, LexToString(Value));
				}
			}
			else if (NumericProperty->IsFloatingPoint())
			{
				double MinValue = 0;
				double MaxValue = 0;

				if (NumericProperty->IsA<FDoubleProperty>())
				{
					MinValue = TNumericLimits<double>::Min();
					MaxValue = TNumericLimits<double>::Max();
				}
				else if (NumericProperty->IsA<FFloatProperty>())
				{
					MinValue = TNumericLimits<float>::Min();
					MaxValue = TNumericLimits<float>::Max();
				}

				if (PropertyHandle->HasMetaData(TEXT("ClampMin")))
				{
					MinValue = FMath::Max(PropertyHandle->GetDoubleMetaData(TEXT("ClampMin")), MinValue);
				}

				if (PropertyHandle->HasMetaData(TEXT("ClampMax")))
				{
					MaxValue = FMath::Min(PropertyHandle->GetDoubleMetaData(TEXT("ClampMax")), MaxValue);
				}

				for (int32 Index = 0; Index < PropertyHandle->GetNumPerObjectValues(); Index++)
				{
					const double Value = FMath::Lerp(MinValue, MaxValue, FMath::FRand());
					PropertyHandle->SetPerObjectValue(Index, LexToString(Value));
				}
			}
		}
	}
	
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
