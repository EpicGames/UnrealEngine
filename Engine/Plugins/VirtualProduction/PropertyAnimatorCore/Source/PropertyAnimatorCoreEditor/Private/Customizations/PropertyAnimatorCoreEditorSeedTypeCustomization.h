// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class FReply;
class IDetailChildrenBuilder;
class IPropertyTypeCustomizationUtils;

/** Only allow property customization with "Seed" metadata */
class FPropertyAnimatorCoreEditorSeedTypeIdentifier : public IPropertyTypeIdentifier
{
public:
	//~ Begin IPropertyTypeIdentifier
	virtual bool IsPropertyTypeCustomized(const IPropertyHandle& InPropertyHandle) const override
	{
		return InPropertyHandle.HasMetaData(TEXT("Seed"));
	}
	//~ End IPropertyTypeIdentifier
};

/** Type customization for seed properties */
class FPropertyAnimatorCoreEditorSeedTypeCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FPropertyAnimatorCoreEditorSeedTypeCustomization>();
	}

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InRow, IPropertyTypeCustomizationUtils& InUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InBuilder, IPropertyTypeCustomizationUtils& InUtils) override;
	//~ End IPropertyTypeCustomization

private:
	static FReply OnGenerateSeedClicked(TWeakPtr<IPropertyHandle> InPropertyHandleWeak);
};
