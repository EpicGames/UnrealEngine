// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/PropertyAnimatorCoreEditorSequencerTimeSourceEvalResultTypeCustomization.h"

#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Selection.h"
#include "Sequencer/MovieSceneAnimatorTrackEditor.h"
#include "TimeSources/PropertyAnimatorCoreSequencerTimeSource.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorCoreEditorSequencerTimeSourceEvalResultTypeCustomization"

void FPropertyAnimatorCoreEditorSequencerTimeSourceChannelTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InRow, IPropertyTypeCustomizationUtils& InUtils)
{
	if (!InPropertyHandle->IsValidHandle())
	{
		return;
	}

	EvalTimePropertyHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPropertyAnimatorCoreSequencerTimeSourceEvalResult, EvalTime));

	if (!EvalTimePropertyHandle->IsValidHandle())
	{
		return;
	}

	InRow.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	];

	InRow.ValueContent()
	.HAlign(HAlign_Fill)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Fill)
		[
			EvalTimePropertyHandle->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(5.f, 0.f)
		.AutoWidth()
		.HAlign(HAlign_Fill)
		[
			SNew(SButton)
			.HAlign(HAlign_Fill)
			.Visibility(this, &FPropertyAnimatorCoreEditorSequencerTimeSourceChannelTypeCustomization::GetCreateTrackButtonVisibility)
			.OnClicked(this, &FPropertyAnimatorCoreEditorSequencerTimeSourceChannelTypeCustomization::OnCreateTrackButtonClicked)
			.IsEnabled(this, &FPropertyAnimatorCoreEditorSequencerTimeSourceChannelTypeCustomization::IsCreateTrackButtonEnabled)
			[
				SNew(STextBlock)
				.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
				.Text(LOCTEXT("AddSequencerTrack", "Create track"))
				.ToolTipText(LOCTEXT("AddSequencerTrackTooltip", "Create a sequencer track linked to this time source"))
			]
		]
	];
}

void FPropertyAnimatorCoreEditorSequencerTimeSourceChannelTypeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InBuilder, IPropertyTypeCustomizationUtils& InUtils)
{
	// No implementation
}

FReply FPropertyAnimatorCoreEditorSequencerTimeSourceChannelTypeCustomization::OnCreateTrackButtonClicked()
{
	const TArray<UObject*> BindingObjects = GetBindingObjects();

	if (BindingObjects.IsEmpty())
	{
		return FReply::Handled();
	}

	TArray<AActor*> SelectedActors = GetSelectedActors();

	FMovieSceneAnimatorTrackEditor::OnAddAnimatorTrack.Broadcast(BindingObjects);

	// Reselect actors after track was created
	if (GEditor)
	{
		GEditor->SelectNone(/** Notify */false, /** DeselectBSP */true);

		for (int32 Index = 0; Index < SelectedActors.Num(); Index++)
		{
			GEditor->SelectActor(SelectedActors[Index], /** Selected */true, /** Notify */Index == SelectedActors.Num() - 1);
		}
	}

	return FReply::Handled();
}

EVisibility FPropertyAnimatorCoreEditorSequencerTimeSourceChannelTypeCustomization::GetCreateTrackButtonVisibility() const
{
	return EvalTimePropertyHandle.IsValid() && EvalTimePropertyHandle->IsValidHandle()
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

bool FPropertyAnimatorCoreEditorSequencerTimeSourceChannelTypeCustomization::IsCreateTrackButtonEnabled() const
{
	const TArray<UObject*> BindingObjects = GetBindingObjects();

	int32 Count = 0;
	FMovieSceneAnimatorTrackEditor::OnGetAnimatorTrackCount.Broadcast(BindingObjects, Count);

	return !BindingObjects.IsEmpty() && BindingObjects.Num() != Count;
}

TArray<AActor*> FPropertyAnimatorCoreEditorSequencerTimeSourceChannelTypeCustomization::GetSelectedActors() const
{
	TArray<AActor*> SelectedActors;

	if (GEditor)
	{
		if (USelection* ActorSelection = GEditor->GetSelectedActors())
		{
			ActorSelection->GetSelectedObjects<AActor>(SelectedActors);
		}
	}

	return SelectedActors;
}

TArray<UObject*> FPropertyAnimatorCoreEditorSequencerTimeSourceChannelTypeCustomization::GetBindingObjects() const
{
	TArray<UObject*> BindingObjects;

	if (!EvalTimePropertyHandle.IsValid() || !EvalTimePropertyHandle->IsValidHandle())
	{
		return BindingObjects;
	}

	EvalTimePropertyHandle->GetOuterObjects(BindingObjects);

	return BindingObjects;
}

#undef LOCTEXT_NAMESPACE
