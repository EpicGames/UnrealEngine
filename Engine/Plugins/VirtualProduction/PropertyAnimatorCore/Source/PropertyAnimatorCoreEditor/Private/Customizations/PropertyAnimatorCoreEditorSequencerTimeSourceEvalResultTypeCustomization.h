// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class AActor;
class FReply;
class IPropertyHandle;
class UObject;
struct EVisibility;

/** Type customization for FPropertyAnimatorCoreSequencerTimeSourceEvalResult */
class FPropertyAnimatorCoreEditorSequencerTimeSourceChannelTypeCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FPropertyAnimatorCoreEditorSequencerTimeSourceChannelTypeCustomization>();
	}

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InRow, IPropertyTypeCustomizationUtils& InUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InBuilder, IPropertyTypeCustomizationUtils& InUtils) override;
	//~ End IPropertyTypeCustomization

protected:
	FReply OnCreateTrackButtonClicked();
	EVisibility GetCreateTrackButtonVisibility() const;
	bool IsCreateTrackButtonEnabled() const;
	TArray<AActor*> GetSelectedActors() const;
	TArray<UObject*> GetBindingObjects() const;

	TSharedPtr<IPropertyHandle> EvalTimePropertyHandle;
};
