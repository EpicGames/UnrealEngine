// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyAnimatorCoreEditorStackCustomization.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Components/PropertyAnimatorCoreComponent.h"
#include "Contexts/OperatorStackEditorMenuContext.h"
#include "Framework/Commands/GenericCommands.h"
#include "Items/OperatorStackEditorGroupItem.h"
#include "Items/OperatorStackEditorObjectItem.h"
#include "Menus/PropertyAnimatorCoreEditorMenu.h"
#include "Presets/PropertyAnimatorCoreAnimatorPreset.h"
#include "Styles/PropertyAnimatorCoreEditorStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/PropertyAnimatorCoreEditorSubsystem.h"
#include "Subsystems/PropertyAnimatorCoreSubsystem.h"
#include "ToolMenus.h"
#include "ToolMenu.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorEditorStackCustomization"

DEFINE_LOG_CATEGORY_STATIC(LogPropertyAnimatorCoreEditorStackCustomization, Log, All);

UPropertyAnimatorCoreEditorStackCustomization::UPropertyAnimatorCoreEditorStackCustomization()
	: UOperatorStackEditorStackCustomization(
		TEXT("Animators")
		, LOCTEXT("CustomizationLabel", "Animators")
		, 0
	)
{
	RegisterCustomizationFor(UPropertyAnimatorCoreBase::StaticClass());
	RegisterCustomizationFor(UPropertyAnimatorCoreComponent::StaticClass());

	// Animator delegates
	UPropertyAnimatorCoreBase::OnPropertyAnimatorAdded().AddUObject(this, &UPropertyAnimatorCoreEditorStackCustomization::OnAnimatorUpdated);
	UPropertyAnimatorCoreBase::OnPropertyAnimatorRemoved().AddUObject(this, &UPropertyAnimatorCoreEditorStackCustomization::OnAnimatorRemoved);
	UPropertyAnimatorCoreBase::OnPropertyAnimatorRenamed().AddUObject(this, &UPropertyAnimatorCoreEditorStackCustomization::OnAnimatorUpdated);
}

UPropertyAnimatorCoreEditorStackCustomization::~UPropertyAnimatorCoreEditorStackCustomization()
{
	UPropertyAnimatorCoreBase::OnPropertyAnimatorAdded().RemoveAll(this);
	UPropertyAnimatorCoreBase::OnPropertyAnimatorRemoved().RemoveAll(this);
	UPropertyAnimatorCoreBase::OnPropertyAnimatorRenamed().RemoveAll(this);
}

bool UPropertyAnimatorCoreEditorStackCustomization::GetRootItem(const FOperatorStackEditorContext& InContext, FOperatorStackEditorItemPtr& OutRootItem) const
{
	TArray<FOperatorStackEditorItemPtr> RootItems;

	// Pick all property animator component as root for the stack view
	for (const FOperatorStackEditorItemPtr& Item : InContext.GetItems())
	{
		if (!Item.IsValid())
		{
			continue;
		}

		if (Item->IsA<AActor>())
		{
			for (const AActor* Actor : Item->GetAsArray<AActor>())
			{
				if (UActorComponent* AnimatorComponent = Actor->FindComponentByClass(UPropertyAnimatorCoreComponent::StaticClass()))
				{
					RootItems.Add(MakeShared<FOperatorStackEditorObjectItem>(AnimatorComponent));
				}
			}
		}
		else if (Item->IsA<UPropertyAnimatorCoreComponent>())
		{
			for (UPropertyAnimatorCoreComponent* Component : Item->GetAsArray<UPropertyAnimatorCoreComponent>())
			{
				RootItems.Add(MakeShared<FOperatorStackEditorObjectItem>(Component));
			}
		}
		else if (Item->IsA<UPropertyAnimatorCoreBase>())
		{
			for (const UPropertyAnimatorCoreBase* Animator : Item->GetAsArray<UPropertyAnimatorCoreBase>())
			{
				RootItems.Add(MakeShared<FOperatorStackEditorObjectItem>(Animator->GetAnimatorComponent()));
			}
		}
	}

	OutRootItem = MakeShared<FOperatorStackEditorGroupItem>(RootItems, FOperatorStackEditorItemType(UPropertyAnimatorCoreComponent::StaticClass(), EOperatorStackEditorItemType::Object));

	return Super::GetRootItem(InContext, OutRootItem);
}

bool UPropertyAnimatorCoreEditorStackCustomization::GetChildrenItem(const FOperatorStackEditorItemPtr& InItem, TArray<FOperatorStackEditorItemPtr>& OutChildrenItems) const
{
	if (InItem->IsA<UPropertyAnimatorCoreComponent>())
	{
		if (InItem->GetValueCount() > 1)
		{
			TMap<UClass*, int32> ClassToIndex;
			TArray<TArray<FOperatorStackEditorItemPtr>> Animators;

			for (const UPropertyAnimatorCoreComponent* AnimatorComponent : InItem->GetAsArray<UPropertyAnimatorCoreComponent>())
			{
				for (UPropertyAnimatorCoreBase* Animator : AnimatorComponent->GetAnimators())
				{
					if (const int32* Index = ClassToIndex.Find(Animator->GetClass()))
					{
						Animators[*Index].Add(MakeShared<FOperatorStackEditorObjectItem>(Animator));
					}
					else
					{
						TArray<FOperatorStackEditorItemPtr> ModifierGroup;
						ModifierGroup.Add(MakeShared<FOperatorStackEditorObjectItem>(Animator));
						int32 GroupIndex = Animators.Add(ModifierGroup);
						ClassToIndex.Add(Animator->GetClass(), GroupIndex);
					}
				}
			}

			for (int32 Index = 0; Index < Animators.Num(); Index++)
			{
				const UClass* const* AnimatorClass = ClassToIndex.FindKey(Index);
				OutChildrenItems.Add(MakeShared<FOperatorStackEditorGroupItem>(Animators[Index], FOperatorStackEditorItemType(*AnimatorClass, EOperatorStackEditorItemType::Object)));
			}
		}
		else
		{
			for (UPropertyAnimatorCoreBase* Animator : InItem->Get<UPropertyAnimatorCoreComponent>(0)->GetAnimators())
			{
				OutChildrenItems.Add(MakeShared<FOperatorStackEditorObjectItem>(Animator));
			}
		}
	}

	return Super::GetChildrenItem(InItem, OutChildrenItems);
}

void UPropertyAnimatorCoreEditorStackCustomization::CustomizeStackHeader(const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorHeaderBuilder& InHeaderBuilder)
{
	if (!InItemTree.GetContext().GetItems().IsEmpty())
	{
		static const FName AddAnimatorMenuName = TEXT("AddAnimatorMenu");
		if (!UToolMenus::Get()->IsMenuRegistered(AddAnimatorMenuName))
		{
			UToolMenu* const AddAnimatorMenu = UToolMenus::Get()->RegisterMenu(AddAnimatorMenuName, NAME_None, EMultiBoxType::Menu);
			AddAnimatorMenu->AddDynamicSection(TEXT("FillAddAnimatorMenuSection"), FNewToolMenuDelegate::CreateUObject(this, &UPropertyAnimatorCoreEditorStackCustomization::FillAddAnimatorMenuSection));
		}

		// Pinned search keywords
		TSet<FString> PinnedAnimatorNames;
		for (const FOperatorStackEditorItemPtr& SupportedItem : InItemTree.GetAllItems())
		{
			if (SupportedItem.IsValid() && SupportedItem->IsA<UPropertyAnimatorCoreBase>())
			{
				if (const UPropertyAnimatorCoreBase* Animator = SupportedItem->Get<UPropertyAnimatorCoreBase>(0))
				{
					PinnedAnimatorNames.Add(Animator->GetAnimatorMetadata()->Name.ToString());
				}
			}
		}

		InHeaderBuilder
			.SetToolMenu(
				AddAnimatorMenuName
				, LOCTEXT("AddAnimatorsMenu", "Add Animators")
				, FAppStyle::GetBrush("Icons.Plus")
			)
			.SetSearchAllowed(true)
			.SetSearchPinnedKeywords(PinnedAnimatorNames);
	}

	Super::CustomizeStackHeader(InItemTree, InHeaderBuilder);
}

void UPropertyAnimatorCoreEditorStackCustomization::CustomizeItemHeader(const FOperatorStackEditorItemPtr& InItem, const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorHeaderBuilder& InHeaderBuilder)
{
	// Action menu available in header in slim toolbar
	static const FName HeaderAnimatorMenuName = TEXT("HeaderAnimatorMenu");
	if (!UToolMenus::Get()->IsMenuRegistered(HeaderAnimatorMenuName))
	{
		UToolMenu* const HeaderModifierMenu = UToolMenus::Get()->RegisterMenu(HeaderAnimatorMenuName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		HeaderModifierMenu->AddDynamicSection(TEXT("FillHeaderAnimatorMenu"), FNewToolMenuDelegate::CreateUObject(this, &UPropertyAnimatorCoreEditorStackCustomization::FillAnimatorHeaderActionMenu));
	}

	// Context menu available when right clicking on item
	static const FName ContextAnimatorMenuName = TEXT("ContextAnimatorMenu");
	if (!UToolMenus::Get()->IsMenuRegistered(ContextAnimatorMenuName))
	{
		UToolMenu* const ContextModifierMenu = UToolMenus::Get()->RegisterMenu(ContextAnimatorMenuName, NAME_None, EMultiBoxType::Menu);
		ContextModifierMenu->AddDynamicSection(TEXT("FillContextAnimatorMenu"), FNewToolMenuDelegate::CreateUObject(this, &UPropertyAnimatorCoreEditorStackCustomization::FillAnimatorContextActionMenu));
	}

	// Customize component header
	if (InItem->IsA<UPropertyAnimatorCoreComponent>())
	{
		FBoolProperty* EnableProperty = FindFProperty<FBoolProperty>(UPropertyAnimatorCoreComponent::StaticClass(), UPropertyAnimatorCoreComponent::GetAnimatorsEnabledPropertyName());

		const FSlateIcon ClassIcon = FSlateIconFinder::FindIconForClass(UPropertyAnimatorCoreComponent::StaticClass());

		// Commands for item on key events
		const TSharedPtr<FUICommandList> ComponentCommands = CreateAnimatorCommands(InItem);

		FText HeaderLabel = LOCTEXT("AnimatorComponentHeaderLabel", "Animators");
		TAttribute<EOperatorStackEditorMessageType> MessageType = EOperatorStackEditorMessageType::None;
		TAttribute<FText> MessageText = FText::GetEmpty();

		if (InItem->GetValueCount() > 1)
		{
			HeaderLabel = FText::Format(LOCTEXT("AnimatorsHeaderLabel", "{0} ({1})"), HeaderLabel, FText::AsNumber(InItem->GetValueCount()));
			MessageType = EOperatorStackEditorMessageType::Info;
			MessageText = LOCTEXT("MultiAnimatorView", "You are viewing multiple items");
		}

		InHeaderBuilder
			.SetProperty(EnableProperty)
			.SetIcon(ClassIcon.GetIcon())
			.SetLabel(HeaderLabel)
			.SetToolbarMenu(HeaderAnimatorMenuName)
			.SetContextMenu(ContextAnimatorMenuName)
			.SetCommandList(ComponentCommands)
			.SetMessageBox(MessageType, MessageText);
	}
	// Customize animator header
	else if (InItem->IsA<UPropertyAnimatorCoreBase>())
	{
		UPropertyAnimatorCoreBase* Animator = InItem->Get<UPropertyAnimatorCoreBase>(0);

		FBoolProperty* EnableProperty = FindFProperty<FBoolProperty>(UPropertyAnimatorCoreBase::StaticClass(), UPropertyAnimatorCoreBase::GetAnimatorEnabledPropertyName());

		const FSlateIcon ClassIcon = FSlateIconFinder::FindIconForClass(Animator->GetClass());

		// Commands for item on key events
		const TSharedPtr<FUICommandList> AnimatorCommands = CreateAnimatorCommands(InItem);

		const FString AnimatorDisplayName = Animator->GetAnimatorDisplayName().ToString();
		const TSharedPtr<const FPropertyAnimatorCoreMetadata> AnimatorMetadata = Animator->GetAnimatorMetadata();

		const TSet<FString> SearchKeywords
		{
			AnimatorMetadata->Name.ToString(),
			AnimatorMetadata->DisplayName.ToString(),
			AnimatorDisplayName
		};

		/** Show last execution error messages if failed execution */
		TAttribute<EOperatorStackEditorMessageType> MessageType = EOperatorStackEditorMessageType::None;
		TAttribute<FText> MessageText = FText::GetEmpty();
		FText HeaderLabel = FText::FromString(TEXT("Animator"));

		if (InItem->GetValueCount() == 1)
		{
			HeaderLabel = AnimatorMetadata->DisplayName;

			TWeakObjectPtr<UPropertyAnimatorCoreBase> AnimatorWeak(Animator);

			MessageType = TAttribute<EOperatorStackEditorMessageType>::CreateLambda([AnimatorWeak]()
			{
				if (const UPropertyAnimatorCoreBase* Animator = AnimatorWeak.Get())
				{
					return Animator->GetLinkedPropertiesCount() > 0
						? EOperatorStackEditorMessageType::None
						: EOperatorStackEditorMessageType::Info;
				}

				return EOperatorStackEditorMessageType::None;
			});

			MessageText = TAttribute<FText>::CreateLambda([AnimatorWeak]()
			{
				if (AnimatorWeak.IsValid())
				{
					return LOCTEXT("NoPropertiesLinked", "No properties are currently linked to this animator");
				}

				return FText::GetEmpty();
			});
		}
		else
		{
			HeaderLabel = FText::Format(LOCTEXT("AnimatorHeaderLabel", "{0} ({1})"), AnimatorMetadata->DisplayName, FText::AsNumber(InItem->GetValueCount()));
		}

		static const FLinearColor AnimatorColor = FLinearColor(FColor::Orange).Desaturate(0.25);

		InHeaderBuilder
			.SetBorderColor(AnimatorColor)
			.SetSearchAllowed(true)
			.SetSearchKeywords(SearchKeywords)
			.SetExpandable(true)
			.SetIcon(ClassIcon.GetIcon())
			.SetLabel(HeaderLabel)
			.SetProperty(EnableProperty)
			.SetCommandList(AnimatorCommands)
			.SetToolbarMenu(HeaderAnimatorMenuName)
			.SetContextMenu(ContextAnimatorMenuName)
			.SetMessageBox(MessageType, MessageText);
	}

	Super::CustomizeItemHeader(InItem, InItemTree, InHeaderBuilder);
}

void UPropertyAnimatorCoreEditorStackCustomization::CustomizeItemBody(const FOperatorStackEditorItemPtr& InItem, const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorBodyBuilder& InBodyBuilder)
{
	if (InItem->IsA<UPropertyAnimatorCoreComponent>())
	{
		FProperty* AnimatorsEnabledProperty = FindFProperty<FProperty>(UPropertyAnimatorCoreComponent::StaticClass(), UPropertyAnimatorCoreComponent::GetAnimatorsEnabledPropertyName());
		FProperty* PropertyAnimatorsProperty = FindFProperty<FProperty>(UPropertyAnimatorCoreComponent::StaticClass(), UPropertyAnimatorCoreComponent::GetPropertyAnimatorsPropertyName());

		InBodyBuilder
			.DisallowProperty(AnimatorsEnabledProperty)
			.DisallowProperty(PropertyAnimatorsProperty)
			.SetShowDetailsView(true);
	}
	// Customize animator body
	else if (InItem->IsA<UPropertyAnimatorCoreBase>())
	{
		FBoolProperty* EnableProperty = FindFProperty<FBoolProperty>(UPropertyAnimatorCoreBase::StaticClass(), UPropertyAnimatorCoreBase::GetAnimatorEnabledPropertyName());
		FProperty* LinkedPropertiesProperty = FindFProperty<FProperty>(UPropertyAnimatorCoreBase::StaticClass(), UPropertyAnimatorCoreBase::GetLinkedPropertiesPropertyName());

		InBodyBuilder
			.SetShowDetailsView(true)
			.DisallowProperty(EnableProperty)
			.ExpandProperty(LinkedPropertiesProperty);
	}

	Super::CustomizeItemBody(InItem, InItemTree, InBodyBuilder);
}

bool UPropertyAnimatorCoreEditorStackCustomization::OnIsItemSelectable(const FOperatorStackEditorItemPtr& InItem)
{
	if (InItem->IsA<UPropertyAnimatorCoreBase>())
	{
		return true;
	}

	return Super::OnIsItemDraggable(InItem);
}

const FSlateBrush* UPropertyAnimatorCoreEditorStackCustomization::GetIcon() const
{
	return FSlateIconFinder::FindIconForClass(UPropertyAnimatorCoreBase::StaticClass()).GetIcon();
}

bool UPropertyAnimatorCoreEditorStackCustomization::ShouldFocusCustomization(const FOperatorStackEditorContext& InContext) const
{
	if (InContext.GetItems().IsEmpty())
	{
		return false;
	}
	
	return InContext.GetItems().Last().IsValid()
		&& (InContext.GetItems().Last()->IsA<UPropertyAnimatorCoreComponent>() || InContext.GetItems().Last()->IsA<UPropertyAnimatorCoreBase>());
}

void UPropertyAnimatorCoreEditorStackCustomization::RemoveAnimatorAction(FOperatorStackEditorItemPtr InItem) const
{
	if (!InItem.IsValid() || !InItem->HasValue())
	{
		return;
	}

	const UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();

	if (!Subsystem)
	{
		return;
	}

	if (InItem->IsA<UPropertyAnimatorCoreBase>())
	{
		const TSet<UPropertyAnimatorCoreBase*> Animators(InItem->GetAsArray<UPropertyAnimatorCoreBase>());

		if (!Subsystem->RemoveAnimators(Animators, /** Transact */true))
		{
			UE_LOGF(LogPropertyAnimatorCoreEditorStackCustomization, Warning, "Could not remove %i animator(s)", Animators.Num())
		}
	}
	else if (InItem->IsA<UPropertyAnimatorCoreComponent>())
	{
		const TSet<UPropertyAnimatorCoreComponent*> Components(InItem->GetAsArray<UPropertyAnimatorCoreComponent>());

		if (!Subsystem->RemoveAnimatorComponents(Components, /** Transact */true))
		{
			UE_LOGF(LogPropertyAnimatorCoreEditorStackCustomization, Warning, "Could not remove %i animator component(s)", Components.Num())
		}
	}
}

bool UPropertyAnimatorCoreEditorStackCustomization::CanExportAnimator(FOperatorStackEditorItemPtr InItem) const
{
	return InItem.IsValid() && InItem->GetValueCount() == 1 && InItem->HasValue(0);
}

void UPropertyAnimatorCoreEditorStackCustomization::ExportAnimatorAction(FOperatorStackEditorItemPtr InItem)
{
	if (!CanExportAnimator(InItem))
	{
		return;
	}

	if (UPropertyAnimatorCoreEditorSubsystem* AnimatorEditorSubsystem = UPropertyAnimatorCoreEditorSubsystem::Get())
	{
		AnimatorEditorSubsystem->CreatePresetAsset(UPropertyAnimatorCoreAnimatorPreset::StaticClass(), {InItem->Get<UPropertyAnimatorCoreBase>(0)});
	}
}

void UPropertyAnimatorCoreEditorStackCustomization::FillAddAnimatorMenuSection(UToolMenu* InToolMenu) const
{
	if (!InToolMenu)
	{
		return;
	}

	const UOperatorStackEditorMenuContext* const AddAnimatorContext = InToolMenu->FindContext<UOperatorStackEditorMenuContext>();
	if (!AddAnimatorContext)
	{
		return;
	}

	const FOperatorStackEditorContextPtr Context = AddAnimatorContext->GetContext();
	if (!Context)
	{
		return;
	}

	UPropertyAnimatorCoreEditorSubsystem* AnimatorEditorSubsystem = UPropertyAnimatorCoreEditorSubsystem::Get();
	if (!AnimatorEditorSubsystem)
	{
		return;
	}

	TSet<UObject*> ContextObjects;
	for (const FOperatorStackEditorItemPtr& ContextItem : Context->GetItems())
	{
		if (ContextItem->IsA<UObject>())
		{
			for (UObject* ContextObject : ContextItem->GetAsArray<UObject>())
			{
				ContextObjects.Add(ContextObject);
			}
		}
	}

	const FPropertyAnimatorCoreEditorMenuContext MenuContext(ContextObjects, {});
	const FPropertyAnimatorCoreEditorMenuOptions MenuOptions({EPropertyAnimatorCoreEditorMenuType::NewSimple});
	AnimatorEditorSubsystem->FillAnimatorMenu(InToolMenu, MenuContext, MenuOptions);
}

void UPropertyAnimatorCoreEditorStackCustomization::FillAnimatorHeaderActionMenu(UToolMenu* InToolMenu)
{
	if (!InToolMenu)
	{
		return;
	}

	const UOperatorStackEditorMenuContext* const MenuContext = InToolMenu->FindContext<UOperatorStackEditorMenuContext>();
	if (!MenuContext)
	{
		return;
	}

	const FOperatorStackEditorItemPtr ItemContext = MenuContext->GetItem();
	if (!ItemContext)
	{
		return;
	}

	if (!ItemContext->IsA<UPropertyAnimatorCoreBase>() && !ItemContext->IsA<UPropertyAnimatorCoreComponent>())
	{
		return;
	}

	if (ItemContext->IsA<UPropertyAnimatorCoreBase>())
	{
		const FToolMenuEntry ExportAnimatorAction = FToolMenuEntry::InitToolBarButton(
			TEXT("ExportAnimatorMenuEntry")
			, FUIAction(
				FExecuteAction::CreateUObject(this, &UPropertyAnimatorCoreEditorStackCustomization::ExportAnimatorAction, ItemContext)
				, FCanExecuteAction::CreateUObject(this, &UPropertyAnimatorCoreEditorStackCustomization::CanExportAnimator, ItemContext)
				, FIsActionChecked()
				, FIsActionButtonVisible::CreateUObject(this, &UPropertyAnimatorCoreEditorStackCustomization::CanExportAnimator, ItemContext))
			, FText::GetEmpty()
			, FText::GetEmpty()
			, FSlateIcon(FPropertyAnimatorCoreEditorStyle::Get().GetStyleSetName(), "PropertyControlIcon.Export")
		);

		InToolMenu->AddMenuEntry(ExportAnimatorAction.Name, ExportAnimatorAction);
	}

	const FToolMenuEntry RemoveAnimatorAction = FToolMenuEntry::InitToolBarButton(
		TEXT("RemoveAnimatorMenuEntry")
		, FExecuteAction::CreateUObject(this, &UPropertyAnimatorCoreEditorStackCustomization::RemoveAnimatorAction, ItemContext)
		, FText::GetEmpty()
		, FText::GetEmpty()
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete")
	);

	InToolMenu->AddMenuEntry(RemoveAnimatorAction.Name, RemoveAnimatorAction);
}

void UPropertyAnimatorCoreEditorStackCustomization::FillAnimatorContextActionMenu(UToolMenu* InToolMenu) const
{
	if (!InToolMenu)
	{
		return;
	}

	const UOperatorStackEditorMenuContext* const MenuContext = InToolMenu->FindContext<UOperatorStackEditorMenuContext>();
	if (!MenuContext)
	{
		return;
	}

	const FOperatorStackEditorItemPtr ItemContext = MenuContext->GetItem();
	if (!ItemContext)
	{
		return;
	}

	if (!ItemContext->IsA<UPropertyAnimatorCoreBase>() && !ItemContext->IsA<UPropertyAnimatorCoreComponent>())
	{
		return;
	}

	// Lets get the commands list to bind the menu entry and execute it when clicked
	TSharedPtr<const FUICommandList> Commands;
	const TSharedPtr<FUICommandInfo> DeleteCommand = FGenericCommands::Get().Delete;
	InToolMenu->Context.GetActionForCommand(DeleteCommand, Commands);
	const FToolMenuEntry RemoveAnimatorMenuEntry = FToolMenuEntry::InitMenuEntryWithCommandList(DeleteCommand, Commands);

	InToolMenu->AddMenuEntry(RemoveAnimatorMenuEntry.Name, RemoveAnimatorMenuEntry);
}

void UPropertyAnimatorCoreEditorStackCustomization::OnAnimatorUpdated(UPropertyAnimatorCoreComponent* InComponent, UPropertyAnimatorCoreBase* InAnimator, EPropertyAnimatorCoreUpdateEvent InType)
{
	if (InComponent)
	{
		RefreshActiveSelection(InComponent, /** Force */false);

		if (InAnimator && InType == EPropertyAnimatorCoreUpdateEvent::User)
		{
			FocusCustomization(InComponent);
		}
	}
}

void UPropertyAnimatorCoreEditorStackCustomization::OnAnimatorRemoved(UPropertyAnimatorCoreComponent* InComponent, UPropertyAnimatorCoreBase* InAnimator, EPropertyAnimatorCoreUpdateEvent InType)
{
	if (InComponent)
	{
		RefreshActiveSelection(InComponent, /** Force */true);
	}
}

TSharedRef<FUICommandList> UPropertyAnimatorCoreEditorStackCustomization::CreateAnimatorCommands(FOperatorStackEditorItemPtr InItem)
{
	TSharedRef<FUICommandList> Commands = MakeShared<FUICommandList>();

	Commands->MapAction(FGenericCommands::Get().Delete,
		FUIAction(
			FExecuteAction::CreateUObject(this, &UPropertyAnimatorCoreEditorStackCustomization::RemoveAnimatorAction, InItem),
			FCanExecuteAction()
		)
	);

	return Commands;
}

#undef LOCTEXT_NAMESPACE
