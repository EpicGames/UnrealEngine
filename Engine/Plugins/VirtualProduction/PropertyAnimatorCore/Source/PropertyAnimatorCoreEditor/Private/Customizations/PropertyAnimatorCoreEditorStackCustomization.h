// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/OperatorStackEditorStackCustomization.h"
#include "PropertyAnimatorCoreEditorStackCustomization.generated.h"

class UPropertyAnimatorCoreComponent;
class UPropertyAnimatorCoreBase;
class UToolMenu;
enum class EPropertyAnimatorCoreUpdateEvent : uint8;

/** Property Controller customization for operator stack tab */
UCLASS()
class UPropertyAnimatorCoreEditorStackCustomization : public UOperatorStackEditorStackCustomization
{
	GENERATED_BODY()

public:
	UPropertyAnimatorCoreEditorStackCustomization();
	virtual ~UPropertyAnimatorCoreEditorStackCustomization() override;

	//~ Begin UOperatorStackEditorStackCustomization
	virtual bool GetRootItem(const FOperatorStackEditorContext& InContext, FOperatorStackEditorItemPtr& OutRootItem) const override;
	virtual bool GetChildrenItem(const FOperatorStackEditorItemPtr& InItem, TArray<FOperatorStackEditorItemPtr>& OutChildrenItems) const override;
	virtual void CustomizeStackHeader(const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorHeaderBuilder& InHeaderBuilder) override;
	virtual void CustomizeItemHeader(const FOperatorStackEditorItemPtr& InItem, const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorHeaderBuilder& InHeaderBuilder) override;
	virtual void CustomizeItemBody(const FOperatorStackEditorItemPtr& InItem, const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorBodyBuilder& InBodyBuilder) override;
	virtual bool OnIsItemSelectable(const FOperatorStackEditorItemPtr& InItem) override;
	virtual const FSlateBrush* GetIcon() const override;
	virtual bool ShouldFocusCustomization(const FOperatorStackEditorContext& InContext) const override;
	//~ End UOperatorStackEditorStackCustomization

protected:
	/** Remove animator menu action */
	void RemoveAnimatorAction(FOperatorStackEditorItemPtr InItem) const;

	bool CanExportAnimator(FOperatorStackEditorItemPtr InItem) const;
	void ExportAnimatorAction(FOperatorStackEditorItemPtr InItem);

	/** Fill item action menus */
	void FillAddAnimatorMenuSection(UToolMenu* InToolMenu) const;
	void FillAnimatorHeaderActionMenu(UToolMenu* InToolMenu);
	void FillAnimatorContextActionMenu(UToolMenu* InToolMenu) const;

	void OnAnimatorUpdated(UPropertyAnimatorCoreComponent* InComponent, UPropertyAnimatorCoreBase* InAnimator, EPropertyAnimatorCoreUpdateEvent InType);
	void OnAnimatorRemoved(UPropertyAnimatorCoreComponent* InComponent, UPropertyAnimatorCoreBase* InAnimator, EPropertyAnimatorCoreUpdateEvent InType);

	/** Create commands for animator */
	TSharedRef<FUICommandList> CreateAnimatorCommands(FOperatorStackEditorItemPtr InItem);
};