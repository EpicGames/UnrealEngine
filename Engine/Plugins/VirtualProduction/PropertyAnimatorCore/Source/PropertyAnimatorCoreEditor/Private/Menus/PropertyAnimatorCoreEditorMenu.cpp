// Copyright Epic Games, Inc. All Rights Reserved.

#include "Menus/PropertyAnimatorCoreEditorMenu.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Components/PropertyAnimatorCoreComponent.h"
#include "Engine/World.h"
#include "Misc/Optional.h"
#include "PropertyHandle.h"
#include "Presets/PropertyAnimatorCoreAnimatorPreset.h"
#include "Presets/PropertyAnimatorCorePresetBase.h"
#include "Presets/PropertyAnimatorCorePropertyPreset.h"
#include "ScopedTransaction.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/PropertyAnimatorCoreEditorSubsystem.h"
#include "Subsystems/PropertyAnimatorCoreSubsystem.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorCoreEditorMenu"

void UE::PropertyAnimatorCoreEditor::Menu::FillNewAnimatorSection(UToolMenu* InMenu, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	if (!InMenu || InMenuData->GetContext().IsEmpty())
	{
		return;
	}

	const UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();
	const UPropertyAnimatorCoreEditorSubsystem* EditorSubsystem = UPropertyAnimatorCoreEditorSubsystem::Get();
	if (!Subsystem || !EditorSubsystem)
	{
		return;
	}

	TSet<UPropertyAnimatorCoreBase*> NewAvailableAnimators = Subsystem->GetAvailableAnimators();
	for (const FPropertyAnimatorCoreData& Property : InMenuData->GetContext().GetProperties())
	{
		NewAvailableAnimators = NewAvailableAnimators.Intersect(Subsystem->GetAvailableAnimators(&Property));
	}

	constexpr bool bCloseMenuAfterSelection = false;
	constexpr bool bOpenOnClick = false;
	const bool bAdvancedMenu = InMenuData->GetOptions().IsMenuType(EPropertyAnimatorCoreEditorMenuType::NewAdvanced);
	const TSet<AActor*>& ContextActors = InMenuData->GetContext().GetActors();

	FToolMenuSection& NewAnimatorsSection = InMenu->FindOrAddSection(TEXT("NewAnimators"), LOCTEXT("NewAnimators.Label", "New Animators"));

	TMap<UPropertyAnimatorCoreBase*, TArray<UPropertyAnimatorCoreAnimatorPreset*>> AvailablePresetAnimators;

	UPropertyAnimatorCoreAnimatorPreset* EmptyPreset = nullptr;
	for (UPropertyAnimatorCoreBase* NewAnimator : NewAvailableAnimators)
	{
		AvailablePresetAnimators.FindOrAdd(NewAnimator).Add(EmptyPreset);
	}

	for (UPropertyAnimatorCorePresetBase* Preset : Subsystem->GetAvailablePresets(UPropertyAnimatorCoreAnimatorPreset::StaticClass()))
	{
		if (UPropertyAnimatorCoreAnimatorPreset* AnimatorPreset = Cast<UPropertyAnimatorCoreAnimatorPreset>(Preset))
		{
			if (UPropertyAnimatorCoreBase* AnimatorTemplate = AnimatorPreset->GetAnimatorTemplate())
			{
				AvailablePresetAnimators.FindOrAdd(AnimatorTemplate).Add(AnimatorPreset);
			}
		}
	}

	for (const TPair<UPropertyAnimatorCoreBase*, TArray<UPropertyAnimatorCoreAnimatorPreset*>>& NewPresetAnimator : AvailablePresetAnimators)
	{
		UPropertyAnimatorCoreBase* NewAnimator = NewPresetAnimator.Key;

		const FName MenuName = NAME_None;
		const TSharedPtr<const FPropertyAnimatorCoreMetadata> Metadata = NewAnimator->GetAnimatorMetadata();
		FName MenuCategory = Metadata->Category;
		FText MenuLabel = Metadata->DisplayName;
		const FText MenuTooltip = LOCTEXT("NewAnimator.Tooltip", "Create a new animator");
		const FSlateIcon MenuIcon = FSlateIconFinder::FindIconForClass(NewAnimator->GetClass());

		UPropertyAnimatorCorePropertyPreset* EmptyPropertyPreset = nullptr;
		for (UPropertyAnimatorCoreAnimatorPreset* AnimatorPreset : NewPresetAnimator.Value)
		{
			if (AnimatorPreset)
			{
				MenuLabel = FText::Format(LOCTEXT("PresetMenuLabel", "{0} ({1})"), AnimatorPreset->GetPresetDisplayName(), Metadata->DisplayName);
				MenuCategory = TEXT("Presets");
			}

			if (bAdvancedMenu)
			{
				NewAnimatorsSection.AddSubMenu(
					MenuName
					, MenuLabel
					, MenuTooltip
					, FNewToolMenuDelegate::CreateLambda(&FillNewAnimatorSubmenu, NewAnimator, AnimatorPreset, InMenuData)
					, bOpenOnClick
					, MenuIcon
					, bCloseMenuAfterSelection
				);
			}
			else
			{
				TSharedPtr<const FPropertyAnimatorCoreEditorCategoryMetadata> AnimatorCategory = EditorSubsystem->FindAnimatorCategory(MenuCategory);
				FToolMenuSection& AnimatorCategorySection = InMenu->FindOrAddSection(AnimatorCategory ? AnimatorCategory->Name : MenuCategory, AnimatorCategory ? AnimatorCategory->DisplayName : FText::FromName(MenuCategory));

				AnimatorCategorySection.AddMenuEntry(
					MenuName
					, MenuLabel
					, MenuTooltip
					, MenuIcon
					, FExecuteAction::CreateLambda(&ExecuteNewAnimatorPresetAction, NewAnimator, ContextActors, AnimatorPreset, EmptyPropertyPreset, InMenuData)
				);
			}
		}
	}
}

void UE::PropertyAnimatorCoreEditor::Menu::FillExistingAnimatorSection(UToolMenu* InMenu, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	if (!InMenu || InMenuData->GetContext().IsEmpty())
	{
		return;
	}

	const UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();
	if (!Subsystem)
	{
		return;
	}

	FToolMenuSection& ExistingAnimatorsSection = InMenu->FindOrAddSection(TEXT("ExistingAnimators"), LOCTEXT("ExistingAnimators.Label", "Existing Animators"));

	constexpr bool bCloseMenuAfterSelection = true;
	constexpr bool bOpenOnClick = false;

	TMap<FName, TSet<UPropertyAnimatorCoreBase*>> ExistingAnimatorsMap;
	for (const FPropertyAnimatorCoreData& Property : InMenuData->GetContext().GetProperties())
	{
		for (AActor* Actor : InMenuData->GetContext().GetActors())
		{
			const FPropertyAnimatorCoreData ActorProperty(Actor, Property.GetLocatorPath());
			for (UPropertyAnimatorCoreBase* Animator : Subsystem->GetExistingAnimators(ActorProperty))
			{
				const FName AnimatorKey = FName(Animator->GetAnimatorDisplayName().ToString() + TEXT(" (") + Animator->GetClass()->GetName() + TEXT(")"));
				ExistingAnimatorsMap.FindOrAdd(AnimatorKey).Add(Animator);
			}
		}
	}

	for (const TPair<FName, TSet<UPropertyAnimatorCoreBase*>>& ExistingAnimators : ExistingAnimatorsMap)
	{
		UPropertyAnimatorCoreBase* Animator = ExistingAnimators.Value.Array()[0];
		const FName MenuName = ExistingAnimators.Key;
		const FText MenuLabel = FText::FromName(ExistingAnimators.Key);
		const FSlateIcon MenuIcon = FSlateIconFinder::FindIconForClass(Animator->GetClass());

		ExistingAnimatorsSection.AddSubMenu(
			MenuName
			, MenuLabel
			, LOCTEXT("ExistingAnimatorSection.Tooltip", "Link or unlink properties for this animator")
			, FNewToolMenuDelegate::CreateLambda(&FillLinkAnimatorSubmenu, ExistingAnimators.Value, InMenuData)
			, bOpenOnClick
			, MenuIcon
			, bCloseMenuAfterSelection
		);
	}
}

void UE::PropertyAnimatorCoreEditor::Menu::FillLinkAnimatorSection(UToolMenu* InMenu, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	if (!InMenu || InMenuData->GetContext().IsEmpty())
	{
		return;
	}

	const UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();
	if (!Subsystem)
	{
		return;
	}

	const TSet<UPropertyAnimatorCoreBase*> Animators = InMenuData->GetContext().GetAnimators();

	if (Animators.IsEmpty())
	{
		return;
	}

	FillLinkAnimatorSubmenu(InMenu, Animators, InMenuData);
}

void UE::PropertyAnimatorCoreEditor::Menu::FillDeleteAnimatorSection(UToolMenu* InMenu, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	if (!InMenu || !InMenuData->GetContext().ContainsAnyComponent())
	{
		return;
	}

	const UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();
	if (!Subsystem)
	{
		return;
	}

	FToolMenuSection& ActorAnimatorsSection = InMenu->FindOrAddSection(TEXT("ActorAnimators"), LOCTEXT("ActorAnimators.Label", "Actor Animators"));

	ActorAnimatorsSection.AddMenuEntry(
		TEXT("DeleteActorAnimator")
		, LOCTEXT("DeleteActorAnimators.Label", "Delete actor animators")
		, LOCTEXT("DeleteActorAnimators.Tooltip", "Delete selected actor animators")
		, FSlateIcon()
		, FUIAction(
		  FExecuteAction::CreateLambda(&ExecuteDeleteActorAnimatorAction, InMenuData)
		)
	);

	const TSet<UPropertyAnimatorCoreBase*>& Animators = InMenuData->GetContext().GetAnimators();

	if (Animators.IsEmpty())
	{
		return;
	}

	ActorAnimatorsSection.AddSeparator(TEXT("ActorAnimatorSeparator"));

	for (UPropertyAnimatorCoreBase* Animator : Animators)
	{
		if (!IsValid(Animator))
		{
			continue;
		}

		ActorAnimatorsSection.AddMenuEntry(
			Animator->GetAnimatorDisplayName()
			, FText::Format(LOCTEXT("DeleteSingleActorAnimator.Label", "Delete {0}"), FText::FromName(Animator->GetAnimatorDisplayName()))
			, LOCTEXT("DeleteSingleActorAnimator.Tooltip", "Delete selected animator")
			, FSlateIcon()
			, FUIAction(
			  FExecuteAction::CreateLambda(&ExecuteDeleteAnimatorAction, Animator, InMenuData)
			)
		);
	}
}

void UE::PropertyAnimatorCoreEditor::Menu::FillEnableAnimatorSection(UToolMenu* InMenu, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	if (!InMenu
		|| InMenuData->GetContext().IsEmpty()
		|| !InMenuData->GetContext().ContainsAnyDisabledAnimator())
	{
		return;
	}

	const UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();
	if (!Subsystem)
	{
		return;
	}

	FToolMenuSection& ActorAnimatorsSection = InMenu->FindOrAddSection(TEXT("ActorAnimators"), LOCTEXT("ActorAnimators.Label", "Actor Animators"));

	constexpr bool bEnable = true;

	ActorAnimatorsSection.AddMenuEntry(
		TEXT("EnableActorAnimator")
		, LOCTEXT("EnableActorAnimator.Label", "Enable actor animators")
		, LOCTEXT("EnableActorAnimator.Tooltip", "Enable selected actor animators")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteEnableActorAnimatorAction, InMenuData, bEnable)
		)
	);

	ActorAnimatorsSection.AddMenuEntry(
		TEXT("EnableLevelAnimator")
		, LOCTEXT("EnableLevelAnimator.Label", "Enable level animators")
		, LOCTEXT("EnableLevelAnimator.Tooltip", "Enable current level animators")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteEnableLevelAnimatorAction, InMenuData, bEnable)
		)
	);

	const TSet<UPropertyAnimatorCoreBase*> DisabledAnimators = InMenuData->GetContext().GetDisabledAnimators();

	if (DisabledAnimators.IsEmpty())
	{
		return;
	}

	ActorAnimatorsSection.AddSeparator(TEXT("ActorAnimatorSeparator"));

	for (UPropertyAnimatorCoreBase* Animator : DisabledAnimators)
	{
		if (!IsValid(Animator))
		{
			continue;
		}

		ActorAnimatorsSection.AddMenuEntry(
			Animator->GetAnimatorDisplayName()
			, FText::Format(LOCTEXT("EnableAnimator.Label", "Enable {0}"), FText::FromName(Animator->GetAnimatorDisplayName()))
			, LOCTEXT("EnableAnimator.Tooltip", "Enable selected animator")
			, FSlateIconFinder::FindIconForClass(Animator->GetClass())
			, FUIAction(
				FExecuteAction::CreateLambda(&ExecuteEnableAnimatorAction, Animator, bEnable, InMenuData)
			)
		);
	}
}

void UE::PropertyAnimatorCoreEditor::Menu::FillDisableAnimatorSection(UToolMenu* InMenu, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	if (!InMenu
		|| InMenuData->GetContext().IsEmpty()
		|| !InMenuData->GetContext().ContainsAnyEnabledAnimator())
	{
		return;
	}

	const UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();
	if (!Subsystem)
	{
		return;
	}

	FToolMenuSection& ActorAnimatorsSection = InMenu->FindOrAddSection(TEXT("ActorAnimators"), LOCTEXT("ActorAnimators.Label", "Actor Animators"));

	constexpr bool bEnable = false;

	ActorAnimatorsSection.AddMenuEntry(
		TEXT("DisableActorAnimator")
		, LOCTEXT("DisableActorAnimator.Label", "Disable actor animators")
		, LOCTEXT("DisableActorAnimator.Tooltip", "Disable selected actor animators")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteEnableActorAnimatorAction, InMenuData, bEnable)
		)
	);

	ActorAnimatorsSection.AddMenuEntry(
		TEXT("DisableLevelAnimator")
		, LOCTEXT("DisableLevelAnimator.Label", "Disable level animators")
		, LOCTEXT("DisableLevelAnimator.Tooltip", "Disable current level animators")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteEnableLevelAnimatorAction, InMenuData, bEnable)
		)
	);

	const TSet<UPropertyAnimatorCoreBase*> EnabledAnimators = InMenuData->GetContext().GetEnabledAnimators();

	if (EnabledAnimators.IsEmpty())
	{
		return;
	}

	ActorAnimatorsSection.AddSeparator(TEXT("ActorAnimatorSeparator"));

	for (UPropertyAnimatorCoreBase* Animator : EnabledAnimators)
	{
		if (!IsValid(Animator))
		{
			continue;
		}

		ActorAnimatorsSection.AddMenuEntry(
			Animator->GetAnimatorDisplayName()
			, FText::Format(LOCTEXT("DisableAnimator.Label", "Disable {0}"), FText::FromName(Animator->GetAnimatorDisplayName()))
			, LOCTEXT("DisableAnimator.Tooltip", "Disable selected animator")
			, FSlateIconFinder::FindIconForClass(Animator->GetClass())
			, FUIAction(
				FExecuteAction::CreateLambda(&ExecuteEnableAnimatorAction, Animator, bEnable, InMenuData)
			)
		);
	}
}

void UE::PropertyAnimatorCoreEditor::Menu::ExecuteNewAnimatorPresetAction(const UPropertyAnimatorCoreBase* InAnimator, const TSet<AActor*>& InActors, UPropertyAnimatorCoreAnimatorPreset* InAnimatorPreset, UPropertyAnimatorCorePropertyPreset* InPropertyPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	const UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();

	if (!Subsystem
		|| !IsValid(InAnimator)
		|| !InAnimator->IsTemplate()
		|| InMenuData->GetContext().IsEmpty())
	{
		return;
	}

	const TSet<UPropertyAnimatorCorePresetBase*> Presets
	{
		InAnimatorPreset,
		InPropertyPreset
	};

	InMenuData->SetLastCreatedAnimators(Subsystem->CreateAnimators(InActors, InAnimator->GetClass(), Presets, InMenuData->GetOptions().ShouldTransact()));
}

void UE::PropertyAnimatorCoreEditor::Menu::ExecuteNewAnimatorPropertyAction(const UPropertyAnimatorCoreBase* InAnimator, const FString& InPropertyLocatorPath, UPropertyAnimatorCoreAnimatorPreset* InAnimatorPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();

	if (!Subsystem
		|| !IsValid(InAnimator)
		|| !InAnimator->IsTemplate()
		|| InMenuData->GetContext().IsEmpty())
	{
		return;
	}

	const TSet<AActor*>& ContextActors = InMenuData->GetContext().GetActors();
	const bool bShouldTransact = InMenuData->GetOptions().ShouldTransact();

	FScopedTransaction Transaction(
		FText::Format(LOCTEXT("CreateAnimatorWithProperty", "Create animator with property on {0} actors"),
			FText::AsNumber(ContextActors.Num()))
		, bShouldTransact && !GIsTransacting);

	TSet<UPropertyAnimatorCoreBase*> CreatedAnimators;
	for (AActor* Actor : ContextActors)
	{
		UPropertyAnimatorCoreBase* NewAnimator = Subsystem->CreateAnimator(Actor, InAnimator->GetClass(), InAnimatorPreset, bShouldTransact);
		FPropertyAnimatorCoreData Property(Actor, InPropertyLocatorPath);
		Subsystem->LinkAnimatorProperty(NewAnimator, Property, bShouldTransact);
		CreatedAnimators.Add(NewAnimator);
	}

	InMenuData->SetLastCreatedAnimators(CreatedAnimators);
}

void UE::PropertyAnimatorCoreEditor::Menu::ExecuteLinkLastCreatedAnimatorPropertyAction(const UPropertyAnimatorCoreBase* InAnimator, const FString& InPropertyLocatorPath, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();

	if (!Subsystem
		|| !IsValid(InAnimator)
		|| !InAnimator->IsTemplate())
	{
		return;
	}

	const TSet<UPropertyAnimatorCoreBase*> LastAnimators = InMenuData->GetLastCreatedAnimators();

	const bool bShouldTransact = InMenuData->GetOptions().ShouldTransact();

	FScopedTransaction Transaction(
		FText::Format(LOCTEXT("LinkPropertyToAnimators", "Toggle link property to {0} animator(s)"),
			FText::AsNumber(LastAnimators.Num()))
		, bShouldTransact && !GIsTransacting);

	for (UPropertyAnimatorCoreBase* LastCreatedAnimator : LastAnimators)
	{
		if (LastCreatedAnimator->GetClass() != InAnimator->GetClass())
		{
			continue;
		}

		FPropertyAnimatorCoreData Property(LastCreatedAnimator->GetAnimatorActor(), InPropertyLocatorPath);
		if (LastCreatedAnimator->IsPropertyLinked(Property))
		{
			Subsystem->UnlinkAnimatorProperty(LastCreatedAnimator, Property, bShouldTransact);
		}
		else
		{
			Subsystem->LinkAnimatorProperty(LastCreatedAnimator, Property, bShouldTransact);
		}
	}
}

void UE::PropertyAnimatorCoreEditor::Menu::ExecuteApplyLastCreatedAnimatorPresetAction(const UPropertyAnimatorCoreBase* InAnimator, UPropertyAnimatorCorePropertyPreset* InPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();

	if (!Subsystem
		|| !IsValid(InAnimator)
		|| !InAnimator->IsTemplate())
	{
		return;
	}

	const TSet<UPropertyAnimatorCoreBase*> LastAnimators = InMenuData->GetLastCreatedAnimators();

	const bool bShouldTransact = InMenuData->GetOptions().ShouldTransact();

	FScopedTransaction Transaction(
		FText::Format(LOCTEXT("ApplyAnimatorsPreset", "Toggle apply preset {0} on {1} animator(s)"),
			InPreset->GetPresetDisplayName(),
			FText::AsNumber(LastAnimators.Num()))
		, bShouldTransact && !GIsTransacting);

	for (UPropertyAnimatorCoreBase* LastCreatedAnimator : LastAnimators)
	{
		if (LastCreatedAnimator->GetClass() != InAnimator->GetClass())
		{
			continue;
		}

		if (InPreset->IsPresetApplied(LastCreatedAnimator))
		{
			Subsystem->UnapplyAnimatorPreset(LastCreatedAnimator, InPreset, bShouldTransact);
		}
		else
		{
			Subsystem->ApplyAnimatorPreset(LastCreatedAnimator, InPreset, bShouldTransact);
		}
	}
}

void UE::PropertyAnimatorCoreEditor::Menu::ExecuteLinkAnimatorPresetAction(const TSet<UPropertyAnimatorCoreBase*>& InAnimators, UPropertyAnimatorCorePropertyPreset* InPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();

	if (!Subsystem
		|| !InPreset
		|| InAnimators.IsEmpty()
		|| InMenuData->GetContext().IsEmpty())
	{
		return;
	}

	const ECheckBoxState State = GetAnimatorPresetState(InAnimators, InPreset);

	const bool bShouldTransact = InMenuData->GetOptions().ShouldTransact();

	FScopedTransaction Transaction(
		FText::Format(LOCTEXT("ApplyAnimatorsPreset", "Toggle apply preset {0} on {1} animator(s)"),
			InPreset->GetPresetDisplayName(),
			FText::AsNumber(InAnimators.Num()))
		, bShouldTransact && !GIsTransacting);

	for (UPropertyAnimatorCoreBase* Animator : InAnimators)
	{
		if (!IsValid(Animator) || Animator->IsTemplate())
		{
			continue;
		}

		if (State == ECheckBoxState::Undetermined || InPreset->IsPresetApplied(Animator))
		{
			Subsystem->UnapplyAnimatorPreset(Animator, InPreset, bShouldTransact);
		}
		else
		{
			Subsystem->ApplyAnimatorPreset(Animator, InPreset, bShouldTransact);
		}
	}
}

void UE::PropertyAnimatorCoreEditor::Menu::ExecuteLinkAnimatorPropertyAction(const TSet<UPropertyAnimatorCoreBase*>& InAnimators, const FString& InPropertyLocatorPath, UPropertyAnimatorCorePropertyPreset* InPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();

	if (!Subsystem
		|| InAnimators.IsEmpty()
		|| InMenuData->GetContext().IsEmpty())
	{
		return;
	}

	const ECheckBoxState State = GetAnimatorPropertyLinkState(InAnimators, InPropertyLocatorPath);

	const bool bShouldTransact = InMenuData->GetOptions().ShouldTransact();

	FScopedTransaction Transaction(
		FText::Format(LOCTEXT("LinkPropertyToAnimators", "Toggle link property to {0} animator(s)"),
			FText::AsNumber(InAnimators.Num()))
		, bShouldTransact && !GIsTransacting);
	
	for (UPropertyAnimatorCoreBase* Animator : InAnimators)
	{
		if (!IsValid(Animator) || Animator->IsTemplate())
		{
			continue;
		}

		FPropertyAnimatorCoreData AnimatorProperty(Animator->GetAnimatorActor(), InPropertyLocatorPath);

		if (!AnimatorProperty.IsResolved() || !Animator->HasPropertySupport(AnimatorProperty))
		{
			continue;
		}

		if (State == ECheckBoxState::Undetermined || Animator->IsPropertyLinked(AnimatorProperty))
		{
			Subsystem->UnlinkAnimatorProperty(Animator, AnimatorProperty, bShouldTransact);

			if (InPreset)
			{
				InPreset->OnPresetUnapplied(Animator, {AnimatorProperty});
			}
		}
		else
		{
			Subsystem->LinkAnimatorProperty(Animator, AnimatorProperty, bShouldTransact);

			if (InPreset)
			{
				InPreset->OnPresetApplied(Animator, {AnimatorProperty});
			}
		}
	}
}

void UE::PropertyAnimatorCoreEditor::Menu::ExecuteEnableActorAnimatorAction(TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData, bool bInEnable)
{
	UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();

	if (InMenuData->GetContext().IsEmpty() || !Subsystem)
	{
		return;
	}

	Subsystem->SetActorAnimatorsEnabled(InMenuData->GetContext().GetActors(), bInEnable, InMenuData->GetOptions().ShouldTransact());
}

void UE::PropertyAnimatorCoreEditor::Menu::ExecuteEnableLevelAnimatorAction(TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData, bool bInEnable)
{
	UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();
	const UWorld* World = InMenuData->GetContext().GetWorld();

	if (!IsValid(World) || !Subsystem)
	{
		return;
	}

	Subsystem->SetLevelAnimatorsEnabled(World, bInEnable, InMenuData->GetOptions().ShouldTransact());
}

void UE::PropertyAnimatorCoreEditor::Menu::ExecuteEnableAnimatorAction(UPropertyAnimatorCoreBase* InAnimator, bool bInEnable, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();

	if (!IsValid(InAnimator) || !Subsystem)
	{
		return;
	}

	Subsystem->SetAnimatorsEnabled({InAnimator}, bInEnable, InMenuData->GetOptions().ShouldTransact());
}

void UE::PropertyAnimatorCoreEditor::Menu::ExecuteDeleteActorAnimatorAction(TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	const UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();

	if (!Subsystem || InMenuData->GetContext().IsEmpty())
	{
		return;
	}

	TSet<UPropertyAnimatorCoreBase*> Animators;

	for (const UPropertyAnimatorCoreComponent* Component : InMenuData->GetContext().GetComponents())
	{
		if (!IsValid(Component))
		{
			continue;
		}

		for (const TObjectPtr<UPropertyAnimatorCoreBase>& Animator : Component->GetAnimators())
		{
			if (Animator)
			{
				Animators.Add(Animator);
			}
		}
	}

	Subsystem->RemoveAnimators(Animators, InMenuData->GetOptions().ShouldTransact());
}

void UE::PropertyAnimatorCoreEditor::Menu::ExecuteDeleteAnimatorAction(UPropertyAnimatorCoreBase* InAnimator, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	const UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();

	if (!IsValid(InAnimator) || !Subsystem)
	{
		return;
	}

	Subsystem->RemoveAnimator(InAnimator, InMenuData->GetOptions().ShouldTransact());
}

ECheckBoxState UE::PropertyAnimatorCoreEditor::Menu::GetAnimatorPresetState(const TSet<UPropertyAnimatorCoreBase*>& InAnimators, UPropertyAnimatorCorePropertyPreset* InPreset)
{
	if (InAnimators.IsEmpty() || !InPreset)
	{
		return ECheckBoxState::Unchecked;
	}

	TSet<FPropertyAnimatorCoreData> SupportedProperties;
	TSet<FPropertyAnimatorCoreData> AppliedProperties;

	for (UPropertyAnimatorCoreBase* Animator : InAnimators)
	{
		if (!IsValid(Animator) || Animator->IsTemplate())
		{
			continue;
		}

		TSet<FPropertyAnimatorCoreData> AnimatorSupportedProperties;
		TSet<FPropertyAnimatorCoreData> AnimatorAppliedProperties;
		InPreset->GetAppliedPresetProperties(Animator, AnimatorSupportedProperties, AnimatorAppliedProperties);

		SupportedProperties.Append(AnimatorSupportedProperties);
		AppliedProperties.Append(AnimatorAppliedProperties);
	}

	if (!SupportedProperties.IsEmpty() && SupportedProperties.Num() == AppliedProperties.Num())
	{
		return ECheckBoxState::Checked;
	}

	return !AppliedProperties.IsEmpty() ? ECheckBoxState::Undetermined : ECheckBoxState::Unchecked;
}

ECheckBoxState UE::PropertyAnimatorCoreEditor::Menu::GetLastAnimatorCreatedPresetState(const UPropertyAnimatorCoreBase* InAnimator, UPropertyAnimatorCorePropertyPreset* InPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	if (!IsValid(InAnimator)
		|| !InAnimator->IsTemplate()
		|| !InPreset
		|| !InMenuData->ContainsAnyLastCreatedAnimator())
	{
		return ECheckBoxState::Unchecked;
	}

	TOptional<ECheckBoxState> AnimatorsState;

	for (UPropertyAnimatorCoreBase* LastCreatedAnimator : InMenuData->GetLastCreatedAnimators())
	{
		if (LastCreatedAnimator->GetClass() == InAnimator->GetClass())
		{
			TSet<UPropertyAnimatorCoreBase*> Animators {LastCreatedAnimator};

			if (!AnimatorsState.IsSet())
			{
				AnimatorsState = GetAnimatorPresetState(Animators, InPreset);
			}
			else if (GetAnimatorPresetState(Animators, InPreset) != AnimatorsState.GetValue())
			{
				AnimatorsState = ECheckBoxState::Undetermined;
				break;
			}
		}
	}

	return AnimatorsState.Get(ECheckBoxState::Unchecked);
}

ECheckBoxState UE::PropertyAnimatorCoreEditor::Menu::GetAnimatorPropertyLinkState(const TSet<UPropertyAnimatorCoreBase*>& InAnimators, const FString& InPropertyLocatorPath)
{
	if (InAnimators.IsEmpty())
	{
		return ECheckBoxState::Undetermined;
	}

	ECheckBoxState State = ECheckBoxState::Undetermined;
	for (const UPropertyAnimatorCoreBase* Animator : InAnimators)
	{
		if (!IsValid(Animator) || Animator->IsTemplate())
		{
			continue;
		}

		ECheckBoxState AnimatorState = ECheckBoxState::Undetermined;
		FPropertyAnimatorCoreData AnimatorProperty(Animator->GetAnimatorActor(), InPropertyLocatorPath);

		if (IsAnimatorPropertyLinked(Animator, AnimatorProperty))
		{
			AnimatorState = ECheckBoxState::Checked;
		}
		else
		{
			AnimatorState = ECheckBoxState::Unchecked;
		}

		if (State == ECheckBoxState::Undetermined)
		{
			State = AnimatorState;
			continue;
		}

		if (AnimatorState != State)
		{
			State = ECheckBoxState::Undetermined;
			break;
		}
	}

	return State;
}

bool UE::PropertyAnimatorCoreEditor::Menu::IsAnimatorPresetLinked(UPropertyAnimatorCoreBase* InAnimator, UPropertyAnimatorCorePropertyPreset* InPreset)
{
	if (!IsValid(InAnimator) || InAnimator->IsTemplate() || !InPreset)
	{
		return false;
	}

	return InPreset->IsPresetApplied(InAnimator);
}

void UE::PropertyAnimatorCoreEditor::Menu::FillNewAnimatorSubmenu(UToolMenu* InMenu, UPropertyAnimatorCoreBase* InAnimator, UPropertyAnimatorCoreAnimatorPreset* InAnimatorPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	const UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();

	if (!InMenu
		|| !Subsystem
		|| !InAnimator
		|| !InAnimator->IsTemplate())
	{
		return;
	}

	if (InMenuData->GetContext().ContainsAnyActor())
	{
		const TSet<AActor*>& ContextActors = InMenuData->GetContext().GetActors();

		FToolMenuSection& PresetSection = InMenu->FindOrAddSection(TEXT("PropertyPresets"), LOCTEXT("NewAnimatorPropertyPresetsSection.Label", "Property Presets"));

		UPropertyAnimatorCorePropertyPreset* EmptyPropertyPreset = nullptr;

		PresetSection.AddMenuEntry(
			TEXT("EmptyPreset")
			, LOCTEXT("NewAnimatorEmptyPresetSection.Label", "Empty")
			, LOCTEXT("NewAnimatorEmptyPresetSection.Tooltip", "Create an empty animator")
			, FSlateIcon()
			, FUIAction(
				FExecuteAction::CreateLambda(&ExecuteNewAnimatorPresetAction, InAnimator, ContextActors, InAnimatorPreset, EmptyPropertyPreset, InMenuData)
				, FCanExecuteAction()
				, FIsActionChecked()
				, FIsActionButtonVisible::CreateLambda(&IsLastAnimatorCreatedActionHidden, InAnimator, InMenuData)
			)
		);

		TSet<UPropertyAnimatorCorePresetBase*> SupportedPropertyPresets = Subsystem->GetAvailablePresets(UPropertyAnimatorCorePropertyPreset::StaticClass());

		for (const AActor* Actor : ContextActors)
		{
			SupportedPropertyPresets = SupportedPropertyPresets.Intersect(Subsystem->GetSupportedPresets(Actor, InAnimator, UPropertyAnimatorCorePropertyPreset::StaticClass()));
		}

		constexpr bool bCloseMenuAfterSelection = false;
		constexpr bool bOpenOnClick = false;

		for (UPropertyAnimatorCorePresetBase* SupportedPropertyPreset : SupportedPropertyPresets)
		{
			UPropertyAnimatorCorePropertyPreset* PropertyPreset = Cast<UPropertyAnimatorCorePropertyPreset>(SupportedPropertyPreset);
			if (!PropertyPreset)
			{
				continue;
			}

			const FString MenuName = SupportedPropertyPreset->GetPresetName().ToString();
			const FText MenuLabel = SupportedPropertyPreset->GetPresetDisplayName();

			PresetSection.AddSubMenu(
				FName(TEXT("Create") + MenuName)
				, MenuLabel
				, LOCTEXT("NewAnimatorPresetSection.Tooltip", "Create this animator using this preset")
				, FNewToolMenuDelegate::CreateLambda(&FillNewPresetAnimatorSubmenu, InAnimator, InAnimatorPreset, PropertyPreset, InMenuData)
				, bOpenOnClick
				, FSlateIcon()
				, bCloseMenuAfterSelection
			);
		}
	}

	if (InMenuData->GetContext().ContainsAnyProperty())
	{
		FToolMenuSection& PropertySection = InMenu->FindOrAddSection(TEXT("Properties"), LOCTEXT("NewAnimatorPropertiesSection.Label", "Properties"));

		TSet<FPropertyAnimatorCoreData> SupportedProperties;
		for (const FPropertyAnimatorCoreData& Property : InMenuData->GetContext().GetProperties())
		{
			InAnimator->GetPropertiesSupported(Property, SupportedProperties, /** SearchDepth */3);
		}

		for (const FPropertyAnimatorCoreData& SupportedProperty : SupportedProperties)
		{
			const FString MenuName = SupportedProperty.GetPropertyDisplayName();
			const FText MenuLabel = FText::FromString(MenuName + TEXT(" (") + SupportedProperty.GetLeafPropertyTypeName().ToString() + TEXT(")"));
			const FString PropertyLocatorPath = SupportedProperty.GetLocatorPath();

			// Create action (creates an animator and links the property)
			PropertySection.AddMenuEntry(
				FName(TEXT("Create") + MenuName)
				, MenuLabel
				, LOCTEXT("NewAnimatorPropertySection.Tooltip", "Create this animator using this property")
				, FSlateIcon()
				, FUIAction(
					FExecuteAction::CreateLambda(&ExecuteNewAnimatorPropertyAction, InAnimator, PropertyLocatorPath, InAnimatorPreset, InMenuData)
					, FCanExecuteAction()
					, FIsActionChecked()
					, FIsActionButtonVisible::CreateLambda(&IsLastAnimatorCreatedActionHidden, InAnimator, InMenuData)
				)
			);

			// Link action (links the property to last created animator)
			PropertySection.AddMenuEntry(
				FName(TEXT("Link") + MenuName)
				, MenuLabel
				, LOCTEXT("LinkLastCreatedAnimatorPropertySection.Tooltip", "Link this property to the last created animator")
				, FSlateIcon()
				, FUIAction(
					FExecuteAction::CreateLambda(&ExecuteLinkLastCreatedAnimatorPropertyAction, InAnimator, PropertyLocatorPath, InMenuData)
					, FCanExecuteAction()
					, FIsActionChecked::CreateLambda(&IsLastAnimatorCreatedPropertyLinked, InAnimator, PropertyLocatorPath, InMenuData)
					, FIsActionButtonVisible::CreateLambda(&IsLastAnimatorCreatedActionVisible, InAnimator, InMenuData)
				)
				, EUserInterfaceActionType::ToggleButton
			);
		}
	}
}

void UE::PropertyAnimatorCoreEditor::Menu::FillLinkAnimatorSubmenu(UToolMenu* InMenu, const TSet<UPropertyAnimatorCoreBase*>& InAnimators, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	if (!InMenu
		|| InAnimators.IsEmpty()
		|| InMenuData->GetContext().IsEmpty())
	{
		return;
	}

	const UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();
	if (!Subsystem)
	{
		return;
	}

	FToolMenuSection& PresetSection = InMenu->FindOrAddSection(TEXT("PropertyPresets"), LOCTEXT("LinkAnimatorPropertyPresetsSection.Label", "Property Presets"));

	TSet<UPropertyAnimatorCorePresetBase*> SupportedPresets = Subsystem->GetAvailablePresets(UPropertyAnimatorCorePropertyPreset::StaticClass());

	for (const UPropertyAnimatorCoreBase* Animator : InAnimators)
	{
		if (!IsValid(Animator) || Animator->IsTemplate())
		{
			continue;
		}

		SupportedPresets = SupportedPresets.Intersect(Subsystem->GetSupportedPresets(Animator->GetAnimatorActor(), Animator, UPropertyAnimatorCorePropertyPreset::StaticClass()));
	}

	constexpr bool bCloseMenuAfterSelection = false;
    constexpr bool bOpenOnClick = false;

	for (UPropertyAnimatorCorePresetBase* SupportedPreset : SupportedPresets)
	{
		if (!SupportedPreset)
		{
			continue;
		}

		const FName MenuName = SupportedPreset->GetPresetName();
		const FText MenuLabel = SupportedPreset->GetPresetDisplayName();
		const FText MenuTooltip = LOCTEXT("LinkAnimatorPresetSection.Tooltip", "Link or unlink a preset from this animator");

		PresetSection.AddSubMenu(
			MenuName
			, MenuLabel
			, MenuTooltip
			, FNewToolMenuDelegate::CreateLambda(&FillPresetAnimatorSubmenu, InAnimators, Cast<UPropertyAnimatorCorePropertyPreset>(SupportedPreset), InMenuData)
			, bOpenOnClick
			, FSlateIcon()
			, bCloseMenuAfterSelection
		);
	}

	FToolMenuSection& PropertySection = InMenu->FindOrAddSection(TEXT("Properties"), LOCTEXT("LinkAnimatorPropertiesSection.Label", "Properties"));

	bool bFirstAnimator = true;
	TSet<FPropertyAnimatorCoreData> SupportedProperties;
	for (const UPropertyAnimatorCoreBase* Animator : InAnimators)
	{
		for (const FPropertyAnimatorCoreData& Property : InMenuData->GetContext().GetProperties())
		{
			TSet<FPropertyAnimatorCoreData> AnimatorSupportedProperties;
			Animator->GetPropertiesSupported(Property, AnimatorSupportedProperties, /** SearchDepth */3);

			if (bFirstAnimator)
			{
				SupportedProperties = AnimatorSupportedProperties;
				bFirstAnimator = false;
			}
			else
			{
				SupportedProperties = SupportedProperties.Intersect(AnimatorSupportedProperties);
			}
		}
	}

	UPropertyAnimatorCorePropertyPreset* Empty = nullptr;
	for (const FPropertyAnimatorCoreData& SupportedProperty : SupportedProperties)
	{
		const FString MenuName = SupportedProperty.GetPropertyDisplayName();
		const FText MenuLabel = FText::FromString(MenuName + TEXT(" (") + SupportedProperty.GetLeafPropertyTypeName().ToString() + TEXT(")"));
		const FString PropertyLocatorPath = SupportedProperty.GetLocatorPath();
		
		PropertySection.AddMenuEntry(
			FName(MenuName)
			, MenuLabel
			, LOCTEXT("LinkAnimatorPropertySection.Tooltip", "Link or unlink this property from the animator")
			, FSlateIcon()
			, FUIAction(
				FExecuteAction::CreateLambda(&ExecuteLinkAnimatorPropertyAction, InAnimators, PropertyLocatorPath, Empty, InMenuData)
				, FCanExecuteAction::CreateLambda(&IsAnimatorLinkPropertyAllowed, InAnimators, PropertyLocatorPath)
				, FGetActionCheckState::CreateLambda(&GetAnimatorPropertyLinkState, InAnimators, PropertyLocatorPath)
			)
			, EUserInterfaceActionType::ToggleButton
		);
	}
}

void UE::PropertyAnimatorCoreEditor::Menu::FillPresetAnimatorSubmenu(UToolMenu* InMenu, const TSet<UPropertyAnimatorCoreBase*>& InAnimators, UPropertyAnimatorCorePropertyPreset* InPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	if (!InMenu)
	{
		return;
	}

	const FToolMenuEntry AllPropertiesEntry = FToolMenuEntry::InitMenuEntry(
		TEXT("All")
		, LOCTEXT("LinkAllPresetProperty.Label", "All")
		, LOCTEXT("LinkAllPresetProperty.Tooltip", "Link all properties from this preset")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteLinkAnimatorPresetAction, InAnimators, InPreset, InMenuData)
			, FCanExecuteAction()
			, FGetActionCheckState::CreateLambda(&GetAnimatorPresetState, InAnimators, InPreset)
		)
		, EUserInterfaceActionType::ToggleButton
	);

	// Add all preset properties option
	InMenu->AddMenuEntry(
		AllPropertiesEntry.Name,
		AllPropertiesEntry
	);

	const FToolMenuEntry SeparatorEntry = FToolMenuEntry::InitSeparator(TEXT("PresetSeparator"));
	InMenu->AddMenuEntry(SeparatorEntry.Name, SeparatorEntry);

	bool bFirstAnimator = true;
	TSet<FPropertyAnimatorCoreData> SupportedProperties;

	for (UPropertyAnimatorCoreBase* Animator : InAnimators)
	{
		if (!IsValid(Animator) || Animator->IsTemplate())
		{
			continue;
		}

		TSet<FPropertyAnimatorCoreData> AnimatorSupportedProperties;
		InPreset->GetSupportedPresetProperties(Animator->GetAnimatorActor(), Animator, AnimatorSupportedProperties);

		if (bFirstAnimator)
		{
			SupportedProperties = AnimatorSupportedProperties;
			bFirstAnimator = false;
		}
		else
		{
			SupportedProperties = SupportedProperties.Intersect(AnimatorSupportedProperties);
		}
	}
	
	for (const FPropertyAnimatorCoreData& SupportedProperty : SupportedProperties)
	{
		const FString MenuName = SupportedProperty.GetPropertyDisplayName();
		const FText MenuLabel = FText::FromString(MenuName + TEXT(" (") + SupportedProperty.GetLeafPropertyTypeName().ToString() + TEXT(")"));
		const FText MenuTooltip = LOCTEXT("LinkPresetProperty.Tooltip", "Link this preset property");
		const FString PropertyLocatorPath = SupportedProperty.GetLocatorPath();

		const FToolMenuEntry SupportedPropertyEntry = FToolMenuEntry::InitMenuEntry(
			FName(MenuName)
			, MenuLabel
			, MenuTooltip
			, FSlateIcon()
			, FUIAction(
				FExecuteAction::CreateLambda(&ExecuteLinkAnimatorPropertyAction, InAnimators, PropertyLocatorPath, InPreset, InMenuData)
				, FCanExecuteAction::CreateLambda(&IsAnimatorLinkPropertyAllowed, InAnimators, PropertyLocatorPath)
				, FGetActionCheckState::CreateLambda(&GetAnimatorPropertyLinkState, InAnimators, PropertyLocatorPath)
			)
			, EUserInterfaceActionType::ToggleButton
		);

		InMenu->AddMenuEntry(SupportedPropertyEntry.Name, SupportedPropertyEntry);
	}
}

void UE::PropertyAnimatorCoreEditor::Menu::FillNewPresetAnimatorSubmenu(UToolMenu* InMenu, UPropertyAnimatorCoreBase* InAnimator, UPropertyAnimatorCoreAnimatorPreset* InAnimatorPreset, UPropertyAnimatorCorePropertyPreset* InPropertyPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	if (!InMenu)
	{
		return;
	}

	const TSet<AActor*>& ContextActors = InMenuData->GetContext().GetActors();

	int32 Index = 0;
	TSet<FPropertyAnimatorCoreData> PresetProperties;
	for (const AActor* ContextActor : ContextActors)
	{
		TSet<FPropertyAnimatorCoreData> SupportedProperties;
		InPropertyPreset->GetSupportedPresetProperties(ContextActor, InAnimator, SupportedProperties);

		if (Index++ == 0)
		{
			PresetProperties = SupportedProperties;
		}
		else
		{
			PresetProperties = PresetProperties.Intersect(SupportedProperties);
		}
	}

	const FToolMenuEntry CreateAllPropertiesEntry = FToolMenuEntry::InitMenuEntry(
		TEXT("CreateAllProperties")
		, LOCTEXT("NewAnimatorPresetSection.Label", "All")
		, LOCTEXT("NewAnimatorPresetSection.Tooltip", "Create this animator using this preset")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteNewAnimatorPresetAction, InAnimator, ContextActors, InAnimatorPreset, InPropertyPreset, InMenuData)
			, FCanExecuteAction()
			, FIsActionChecked()
			, FIsActionButtonVisible::CreateLambda(&IsLastAnimatorCreatedActionHidden, InAnimator, InMenuData)
		)
	);

	// Create action (creates an animator and links the property)
	InMenu->AddMenuEntry(
		CreateAllPropertiesEntry.Name
		, CreateAllPropertiesEntry
	);

	const FToolMenuEntry LinkAllPropertiesEntry = FToolMenuEntry::InitMenuEntry(
		TEXT("LinkAllProperties")
		, LOCTEXT("ApplyLastCreatedAnimatorPresetSection.Label", "All")
		, LOCTEXT("ApplyLastCreatedAnimatorPresetSection.Tooltip", "Apply this preset to the last created animator")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteApplyLastCreatedAnimatorPresetAction, InAnimator, InPropertyPreset, InMenuData)
			, FCanExecuteAction()
			, FGetActionCheckState::CreateLambda(&GetLastAnimatorCreatedPresetState, InAnimator, InPropertyPreset, InMenuData)
			, FIsActionButtonVisible::CreateLambda(&IsLastAnimatorCreatedActionVisible, InAnimator, InMenuData)
		)
		, EUserInterfaceActionType::ToggleButton
	);

	// Link action (links the property to last created animator)
	InMenu->AddMenuEntry(
		LinkAllPropertiesEntry.Name
		, LinkAllPropertiesEntry
	);

	const FToolMenuEntry SeparatorEntry = FToolMenuEntry::InitSeparator(TEXT("PresetSeparator"));
	InMenu->AddMenuEntry(SeparatorEntry.Name, SeparatorEntry);

	for (const FPropertyAnimatorCoreData& PresetProperty : PresetProperties)
	{
		const FString MenuName = PresetProperty.GetPropertyDisplayName();
		const FText MenuLabel = FText::FromString(MenuName + TEXT(" (") + PresetProperty.GetLeafPropertyTypeName().ToString() + TEXT(")"));
		const FString PropertyLocatorPath = PresetProperty.GetLocatorPath();
		
		const FToolMenuEntry CreatePropertyEntry = FToolMenuEntry::InitMenuEntry(
			FName(TEXT("CreateProperty") + MenuName)
			, MenuLabel
			, LOCTEXT("CreateAnimatorPresetProperty.Tooltip", "Create this animator using this preset property")
			, FSlateIcon()
			, FUIAction(
				FExecuteAction::CreateLambda(&ExecuteNewAnimatorPropertyAction, InAnimator, PropertyLocatorPath, InAnimatorPreset, InMenuData)
				, FCanExecuteAction()
				, FIsActionChecked()
				, FIsActionButtonVisible::CreateLambda(&IsLastAnimatorCreatedActionHidden, InAnimator, InMenuData)
			)
		);

		// Create action (creates an animator and links the property)
		InMenu->AddMenuEntry(
			CreatePropertyEntry.Name
			, CreatePropertyEntry
		);

		const FToolMenuEntry LinkPropertyEntry = FToolMenuEntry::InitMenuEntry(
			FName(TEXT("LinkProperty") + MenuName)
			, MenuLabel
			, LOCTEXT("LinkAnimatorPresetProperty.Tooltip", "Link this preset property to the last created animator")
			, FSlateIcon()
			, FUIAction(
				FExecuteAction::CreateLambda(&ExecuteLinkLastCreatedAnimatorPropertyAction, InAnimator, PropertyLocatorPath, InMenuData)
				, FCanExecuteAction()
				, FIsActionChecked::CreateLambda(&IsLastAnimatorCreatedPropertyLinked, InAnimator, PropertyLocatorPath, InMenuData)
				, FIsActionButtonVisible::CreateLambda(&IsLastAnimatorCreatedActionVisible, InAnimator, InMenuData)
			)
			, EUserInterfaceActionType::ToggleButton
		);

		// Link action (links the property to last created animator)
		InMenu->AddMenuEntry(
			LinkPropertyEntry.Name
			, LinkPropertyEntry
		);
	}
}

bool UE::PropertyAnimatorCoreEditor::Menu::IsAnimatorPropertyLinked(const UPropertyAnimatorCoreBase* InAnimator, const FPropertyAnimatorCoreData& InProperty)
{
	if (!IsValid(InAnimator) || InAnimator->IsTemplate())
	{
		return false;
	}

	return InAnimator->IsPropertyLinked(InProperty);
}

bool UE::PropertyAnimatorCoreEditor::Menu::IsAnimatorLinkPropertyAllowed(const TSet<UPropertyAnimatorCoreBase*>& InAnimators, const FString& InPropertyLocatorPath)
{
	if (InAnimators.IsEmpty())
	{
		return false;
	}

	bool bAllowed = false;

	for (const UPropertyAnimatorCoreBase* Animator : InAnimators)
	{
		if (!IsValid(Animator) || Animator->IsTemplate())
		{
			continue;
		}

		FPropertyAnimatorCoreData AnimatorProperty(Animator->GetAnimatorActor(), InPropertyLocatorPath);
		if (!AnimatorProperty.IsResolved())
		{
			continue;
		}

		// Only allow linking properties that are not yet linked and do not have any of their children linked
		bAllowed |= (!Animator->IsPropertyLinked(AnimatorProperty) && Animator->GetInnerPropertiesLinked(AnimatorProperty).IsEmpty()) || Animator->GetLinkedPropertyContext(AnimatorProperty) != nullptr;

		if (!bAllowed)
		{
			break;
		}
	}

	return bAllowed;
}

bool UE::PropertyAnimatorCoreEditor::Menu::IsLastAnimatorCreatedPropertyLinked(const UPropertyAnimatorCoreBase* InAnimator, const FString& InPropertyLocatorPath, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	if (!IsValid(InAnimator)
		|| !InAnimator->IsTemplate()
		|| !InMenuData->ContainsAnyLastCreatedAnimator())
	{
		return false;
	}

	for (const UPropertyAnimatorCoreBase* LastCreatedAnimator : InMenuData->GetLastCreatedAnimators())
	{
		if (LastCreatedAnimator->GetClass() != InAnimator->GetClass())
		{
			return false;
		}

		const FPropertyAnimatorCoreData Property(LastCreatedAnimator->GetAnimatorActor(), InPropertyLocatorPath);
		if (!LastCreatedAnimator->IsPropertyLinked(Property))
		{
			return false;
		}
	}

	return true;
}

bool UE::PropertyAnimatorCoreEditor::Menu::IsLastAnimatorCreatedActionVisible(const UPropertyAnimatorCoreBase* InAnimator, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	if (!IsValid(InAnimator)
		|| !InAnimator->IsTemplate()
		|| !InMenuData->ContainsAnyLastCreatedAnimator())
	{
		return false;
	}

	for (const UPropertyAnimatorCoreBase* LastCreatedAnimator : InMenuData->GetLastCreatedAnimators())
	{
		if (LastCreatedAnimator->GetClass() != InAnimator->GetClass())
		{
			return false;
		}
	}

	return true;
}

bool UE::PropertyAnimatorCoreEditor::Menu::IsLastAnimatorCreatedActionHidden(const UPropertyAnimatorCoreBase* InAnimator, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData)
{
	return !IsLastAnimatorCreatedActionVisible(InAnimator, InMenuData);
}

#undef LOCTEXT_NAMESPACE
