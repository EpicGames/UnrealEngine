// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "Menus/PropertyAnimatorCoreEditorMenuDefs.h"
#include "Presets/PropertyAnimatorCorePresetBase.h"
#include "Properties/PropertyAnimatorCoreData.h"
#include "Templates/SharedPointer.h"
#include "ToolMenus.h"

struct FPropertyAnimatorCoreData;
class IPropertyHandle;
class UPropertyAnimatorCoreBase;
class UPropertyAnimatorCoreEditorMenuContext;
class UPropertyAnimatorCoreAnimatorPreset;
class UPropertyAnimatorCorePropertyPreset;

namespace UE::PropertyAnimatorCoreEditor::Menu
{
	/** Sections */

	void FillNewAnimatorSection(UToolMenu* InMenu, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	void FillExistingAnimatorSection(UToolMenu* InMenu, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	void FillLinkAnimatorSection(UToolMenu* InMenu, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	void FillEnableAnimatorSection(UToolMenu* InMenu, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	void FillDisableAnimatorSection(UToolMenu* InMenu, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	void FillDeleteAnimatorSection(UToolMenu* InMenu, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	/** Submenus */

	void FillNewAnimatorSubmenu(UToolMenu* InMenu, UPropertyAnimatorCoreBase* InAnimator, UPropertyAnimatorCoreAnimatorPreset* InAnimatorPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	void FillLinkAnimatorSubmenu(UToolMenu* InMenu, const TSet<UPropertyAnimatorCoreBase*>& InAnimators, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	void FillPresetAnimatorSubmenu(UToolMenu* InMenu, const TSet<UPropertyAnimatorCoreBase*>& InAnimators, UPropertyAnimatorCorePropertyPreset* InPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	void FillNewPresetAnimatorSubmenu(UToolMenu* InMenu, UPropertyAnimatorCoreBase* InAnimator, UPropertyAnimatorCoreAnimatorPreset* InAnimatorPreset, UPropertyAnimatorCorePropertyPreset* InPropertyPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	/** Execute */

	void ExecuteNewAnimatorPresetAction(const UPropertyAnimatorCoreBase* InAnimator, const TSet<AActor*>& InActors, UPropertyAnimatorCoreAnimatorPreset* InAnimatorPreset, UPropertyAnimatorCorePropertyPreset* InPropertyPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	void ExecuteNewAnimatorPropertyAction(const UPropertyAnimatorCoreBase* InAnimator, const FString& InPropertyLocatorPath, UPropertyAnimatorCoreAnimatorPreset* InAnimatorPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	void ExecuteLinkLastCreatedAnimatorPropertyAction(const UPropertyAnimatorCoreBase* InAnimator, const FString& InPropertyLocatorPath, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	void ExecuteApplyLastCreatedAnimatorPresetAction(const UPropertyAnimatorCoreBase* InAnimator, UPropertyAnimatorCorePropertyPreset* InPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	void ExecuteLinkAnimatorPresetAction(const TSet<UPropertyAnimatorCoreBase*>& InAnimators, UPropertyAnimatorCorePropertyPreset* InPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	void ExecuteLinkAnimatorPropertyAction(const TSet<UPropertyAnimatorCoreBase*>& InAnimators, const FString& InPropertyLocatorPath, UPropertyAnimatorCorePropertyPreset* InPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	void ExecuteEnableActorAnimatorAction(TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData, bool bInEnable);

	void ExecuteEnableLevelAnimatorAction(TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData, bool bInEnable);

	void ExecuteEnableAnimatorAction(UPropertyAnimatorCoreBase* InAnimator, bool bInEnable, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	void ExecuteDeleteActorAnimatorAction(TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	void ExecuteDeleteAnimatorAction(UPropertyAnimatorCoreBase* InAnimator, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	/** Check */

	ECheckBoxState GetAnimatorPresetState(const TSet<UPropertyAnimatorCoreBase*>& InAnimators, UPropertyAnimatorCorePropertyPreset* InPreset);

	ECheckBoxState GetLastAnimatorCreatedPresetState(const UPropertyAnimatorCoreBase* InAnimator, UPropertyAnimatorCorePropertyPreset* InPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	ECheckBoxState GetAnimatorPropertyLinkState(const TSet<UPropertyAnimatorCoreBase*>& InAnimators, const FString& InPropertyLocatorPath);

	bool IsAnimatorPresetLinked(UPropertyAnimatorCoreBase* InAnimator, UPropertyAnimatorCorePropertyPreset* InPreset);

	bool IsAnimatorPropertyLinked(const UPropertyAnimatorCoreBase* InAnimator, const FPropertyAnimatorCoreData& InProperty);

	bool IsAnimatorLinkPropertyAllowed(const TSet<UPropertyAnimatorCoreBase*>& InAnimators, const FString& InPropertyLocatorPath);

	bool IsLastAnimatorCreatedPropertyLinked(const UPropertyAnimatorCoreBase* InAnimator, const FString& InPropertyLocatorPath, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	bool IsLastAnimatorCreatedPresetLinked(const UPropertyAnimatorCoreBase* InAnimator, UPropertyAnimatorCorePropertyPreset* InPreset, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	bool IsLastAnimatorCreatedActionVisible(const UPropertyAnimatorCoreBase* InAnimator, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);

	bool IsLastAnimatorCreatedActionHidden(const UPropertyAnimatorCoreBase* InAnimator, TSharedRef<FPropertyAnimatorCoreEditorMenuData> InMenuData);
}
