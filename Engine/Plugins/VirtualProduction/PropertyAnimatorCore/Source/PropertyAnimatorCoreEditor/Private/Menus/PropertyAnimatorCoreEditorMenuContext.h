// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Properties/PropertyAnimatorCoreData.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "PropertyAnimatorCoreEditorMenuContext.generated.h"

class UPropertyAnimatorCoreBase;
struct FPropertyAnimatorCoreData;

/** Context passed in UToolMenu when generating entries with selected items */
UCLASS()
class UPropertyAnimatorCoreEditorMenuContext : public UObject
{
	GENERATED_BODY()

public:
	const TArray<FPropertyAnimatorCoreData>& GetProperties() const
	{
		return Properties;
	}

	void SetProperties(const TArray<FPropertyAnimatorCoreData>& InProperties)
	{
		Properties = InProperties;
	}

	const TSet<UPropertyAnimatorCoreBase*>& GetAnimators() const
	{
		return Animators;
	}

	void SetAnimators(const TSet<UPropertyAnimatorCoreBase*>& InAnimators)
	{
		Animators = InAnimators;
	}

protected:
	/** The items this menu should apply to */
	TArray<FPropertyAnimatorCoreData> Properties;
	TSet<UPropertyAnimatorCoreBase*> Animators;
};