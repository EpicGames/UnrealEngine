// Copyright Epic Games, Inc. All Rights Reserved.

#include "Menus/PropertyAnimatorCoreEditorMenuDefs.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Components/PropertyAnimatorCoreComponent.h"

FPropertyAnimatorCoreEditorMenuContext::FPropertyAnimatorCoreEditorMenuContext(const TSet<UObject*>& InObjects, const TArray<FPropertyAnimatorCoreData>& InProperties)
{
	auto AddActor = [this](AActor* InActor)
	{
		if (IsValid(InActor))
		{
			ContextActors.Add(InActor);

			if (UPropertyAnimatorCoreComponent* Component = InActor->FindComponentByClass<UPropertyAnimatorCoreComponent>())
			{
				ContextComponents.Add(Component);
			}
		}
	};

	if (!InProperties.IsEmpty())
	{
		for (const FPropertyAnimatorCoreData& Property : InProperties)
		{
			if (Property.IsResolved())
			{
				ContextProperties.Add(Property);
				AddActor(Property.GetOwningActor());
			}
		}
	}

	if (!InObjects.IsEmpty())
	{
		for (UObject* Object : InObjects)
		{
			if (!IsValid(Object))
			{
				continue;
			}

			if (AActor* Actor = Cast<AActor>(Object))
			{
				AddActor(Actor);
			}
			else if (const UPropertyAnimatorCoreComponent* Component = Cast<UPropertyAnimatorCoreComponent>(Object))
			{
				AddActor(Component->GetOwner());
			}
			else if (UPropertyAnimatorCoreBase* Animator = Cast<UPropertyAnimatorCoreBase>(Object))
			{
				ContextAnimators.Add(Animator);
				AddActor(Animator->GetAnimatorActor());
			}
		}
	}
}

const TSet<AActor*>& FPropertyAnimatorCoreEditorMenuContext::GetActors() const
{
	return ContextActors;
}

const TSet<FPropertyAnimatorCoreData>& FPropertyAnimatorCoreEditorMenuContext::GetProperties() const
{
	return ContextProperties;
}

const TSet<UPropertyAnimatorCoreComponent*>& FPropertyAnimatorCoreEditorMenuContext::GetComponents() const
{
	return ContextComponents;
}

const TSet<UPropertyAnimatorCoreBase*>& FPropertyAnimatorCoreEditorMenuContext::GetAnimators() const
{
	return ContextAnimators;
}

TSet<UPropertyAnimatorCoreBase*> FPropertyAnimatorCoreEditorMenuContext::GetDisabledAnimators() const
{
	constexpr bool bAnimatorEnabled = false;
	return GetStateAnimators(bAnimatorEnabled);
}

TSet<UPropertyAnimatorCoreBase*> FPropertyAnimatorCoreEditorMenuContext::GetEnabledAnimators() const
{
	constexpr bool bAnimatorEnabled = true;
	return GetStateAnimators(bAnimatorEnabled);
}

UWorld* FPropertyAnimatorCoreEditorMenuContext::GetWorld() const
{
	for (const AActor* Actor : ContextActors)
	{
		if (IsValid(Actor))
		{
			return Actor->GetWorld();
		}
	}

	return nullptr;
}

bool FPropertyAnimatorCoreEditorMenuContext::IsEmpty() const
{
	return ContextProperties.IsEmpty() && ContextActors.IsEmpty() && ContextAnimators.IsEmpty();
}

bool FPropertyAnimatorCoreEditorMenuContext::ContainsAnyProperty() const
{
	return !ContextProperties.IsEmpty();
}

bool FPropertyAnimatorCoreEditorMenuContext::ContainsAnyActor() const
{
	return !ContextActors.IsEmpty();
}

bool FPropertyAnimatorCoreEditorMenuContext::ContainsAnyAnimator() const
{
	return !ContextAnimators.IsEmpty();
}

bool FPropertyAnimatorCoreEditorMenuContext::ContainsAnyComponent() const
{
	return !ContextComponents.IsEmpty();
}

bool FPropertyAnimatorCoreEditorMenuContext::ContainsAnyDisabledAnimator() const
{
	constexpr bool bAnimatorEnabled = false;
	return ContainsAnimatorState(bAnimatorEnabled);
}

bool FPropertyAnimatorCoreEditorMenuContext::ContainsAnyEnabledAnimator() const
{
	constexpr bool bAnimatorEnabled = true;
	return ContainsAnimatorState(bAnimatorEnabled);
}

bool FPropertyAnimatorCoreEditorMenuContext::ContainsAnyComponentAnimator() const
{
	if (!ContextAnimators.IsEmpty())
	{
		return true;
	}

	for (const UPropertyAnimatorCoreComponent* Component : ContextComponents)
	{
		if (!IsValid(Component))
		{
			continue;
		}

		if (Component->GetAnimatorsCount() > 0)
		{
			return true;
		}
	}

	return false;
}

bool FPropertyAnimatorCoreEditorMenuContext::ContainsAnimatorState(bool bInState) const
{
	for (const UPropertyAnimatorCoreBase* Animator : ContextAnimators)
	{
		if (Animator && Animator->GetAnimatorEnabled() == bInState)
		{
			return true;
		}
	}

	for (const UPropertyAnimatorCoreComponent* AnimatorComponent : ContextComponents)
	{
		if (AnimatorComponent && AnimatorComponent->GetAnimatorsEnabled() == bInState)
		{
			return true;
		}
	}

	return false;
}

TSet<UPropertyAnimatorCoreBase*> FPropertyAnimatorCoreEditorMenuContext::GetStateAnimators(bool bInState) const
{
	TSet<UPropertyAnimatorCoreBase*> FoundAnimators;

	for (UPropertyAnimatorCoreBase* Animator : ContextAnimators)
	{
		if (!IsValid(Animator) || Animator->GetAnimatorEnabled() != bInState)
		{
			continue;
		}

		FoundAnimators.Add(Animator);
	}

	return FoundAnimators;
}

FPropertyAnimatorCoreEditorMenuOptions::FPropertyAnimatorCoreEditorMenuOptions(const TSet<EPropertyAnimatorCoreEditorMenuType>& InMenus)
{
	for (const EPropertyAnimatorCoreEditorMenuType& Menu : InMenus)
	{
		MenuTypes |= static_cast<uint8>(Menu);
	}
}

FPropertyAnimatorCoreEditorMenuOptions& FPropertyAnimatorCoreEditorMenuOptions::CreateSubMenu(bool bInCreateSubMenu)
{
	bCreateSubMenu = bInCreateSubMenu;
	return *this;
}

FPropertyAnimatorCoreEditorMenuOptions& FPropertyAnimatorCoreEditorMenuOptions::UseTransact(bool bInUseTransact)
{
	bUseTransact = bInUseTransact;
	return *this;
}

bool FPropertyAnimatorCoreEditorMenuOptions::IsMenuType(EPropertyAnimatorCoreEditorMenuType InMenuType) const
{
	return EnumHasAnyFlags(static_cast<EPropertyAnimatorCoreEditorMenuType>(MenuTypes), InMenuType);
}

void FPropertyAnimatorCoreEditorMenuData::SetLastCreatedAnimator(UPropertyAnimatorCoreBase* InAnimator)
{
	LastCreatedAnimators = {InAnimator};
}

void FPropertyAnimatorCoreEditorMenuData::SetLastCreatedAnimators(const TSet<UPropertyAnimatorCoreBase*>& InAnimators)
{
	LastCreatedAnimators.Empty(InAnimators.Num());

	Algo::TransformIf(
		InAnimators
		, LastCreatedAnimators
		, [](const UPropertyAnimatorCoreBase* InAnimator)
		{
			return IsValid(InAnimator);
		}
		, [](UPropertyAnimatorCoreBase* InAnimator)
		{
		return InAnimator;
		}
	);
}

TSet<UPropertyAnimatorCoreBase*> FPropertyAnimatorCoreEditorMenuData::GetLastCreatedAnimators() const
{
	TSet<UPropertyAnimatorCoreBase*> Animators;

	Algo::TransformIf(
		LastCreatedAnimators
		, Animators
		, [](const TWeakObjectPtr<UPropertyAnimatorCoreBase>& InAnimatorWeak)
		{
			return InAnimatorWeak.IsValid();
		}
		, [](const TWeakObjectPtr<UPropertyAnimatorCoreBase>& InAnimatorWeak)
		{
			return InAnimatorWeak.Get();
		}
	);

	return Animators;
}

bool FPropertyAnimatorCoreEditorMenuData::ContainsAnyLastCreatedAnimator() const
{
	return !LastCreatedAnimators.IsEmpty();
}

UPropertyAnimatorCoreBase* FPropertyAnimatorCoreEditorMenuData::GetLastCreatedAnimator() const
{
	return !LastCreatedAnimators.IsEmpty() ? LastCreatedAnimators.Array()[0].Get() : nullptr;
}
