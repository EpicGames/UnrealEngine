// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_PropertyAnimatorCorePreset.h"

#include "Presets/PropertyAnimatorCorePresetBase.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_PropertyAnimatorCorePreset"

FText UAssetDefinition_PropertyAnimatorCorePreset::GetAssetDisplayName() const
{
	return LOCTEXT("AssetDisplayName", "PropertyAnimatorCorePreset");
}

TSoftClassPtr<UObject> UAssetDefinition_PropertyAnimatorCorePreset::GetAssetClass() const
{
	return UPropertyAnimatorCorePresetBase::StaticClass();
}

FLinearColor UAssetDefinition_PropertyAnimatorCorePreset::GetAssetColor() const
{
	return FLinearColor::White;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_PropertyAnimatorCorePreset::GetAssetCategories() const
{
	static const TArray<FAssetCategoryPath> Categories = {EAssetCategoryPaths::Animation};
	return Categories;
}

#undef LOCTEXT_NAMESPACE