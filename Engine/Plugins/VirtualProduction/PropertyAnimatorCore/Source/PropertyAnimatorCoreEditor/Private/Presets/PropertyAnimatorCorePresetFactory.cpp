// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyAnimatorCorePresetFactory.h"

#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/SClassPickerDialog.h"
#include "Modules/ModuleManager.h"
#include "Presets/PropertyAnimatorCorePresetBase.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorCorePresetFactory"

UPropertyAnimatorCorePresetFactory::UPropertyAnimatorCorePresetFactory()
{
	SupportedClass = UPropertyAnimatorCorePresetBase::StaticClass();
	bCreateNew = true;
	bEditAfterNew = false;
	bEditorImport = false;
	bText = true;
}

bool UPropertyAnimatorCorePresetFactory::ConfigureProperties()
{
	NewPresetClass = nullptr;

	// Load the class viewer module to display a class picker
	FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	class FAssetClassParentFilter : public IClassViewerFilter
	{
	public:
		FAssetClassParentFilter()
			: DisallowedClassFlags(CLASS_None)
		{}

		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFunc) override
		{
			return !InClass->HasAnyClassFlags(DisallowedClassFlags)
				&& InClass->CanCreateAssetOfClass()
				&& InFilterFunc->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed
				&& !FKismetEditorUtilities::CanCreateBlueprintOfClass(InClass);
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFunc) override
		{
			// Disallow blueprint base class
			return false;
		}

		TSet<const UClass*> AllowedChildrenOfClasses;
		EClassFlags DisallowedClassFlags;
	};

	const TSharedPtr<FAssetClassParentFilter> ClassFilter = MakeShareable(new FAssetClassParentFilter());
	ClassFilter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Transient;
	ClassFilter->AllowedChildrenOfClasses.Add(UPropertyAnimatorCorePresetBase::StaticClass());

	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.ClassFilters.Add(ClassFilter.ToSharedRef());

	const FText TitleText = LOCTEXT("CreatePropertyAnimatorCorePreset", "Pick Preset Class");

	UClass* PickedClass = nullptr;
	const bool bSuccess = SClassPickerDialog::PickClass(TitleText, Options, PickedClass, UPropertyAnimatorCorePresetBase::StaticClass());

	if (bSuccess)
	{
		NewPresetClass = PickedClass;
	}

	return bSuccess;
}

UObject* UPropertyAnimatorCorePresetFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn)
{
	UPropertyAnimatorCorePresetBase* NewObjectAsset = nullptr;

	if (NewPresetClass)
	{
		NewObjectAsset = NewObject<UPropertyAnimatorCorePresetBase>(InParent, NewPresetClass, InName, InFlags | RF_Transactional);
	}

	return NewObjectAsset;
}

UObject* UPropertyAnimatorCorePresetFactory::FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags,
	UObject* InContext, const TCHAR* InType, const TCHAR*& InBuffer, const TCHAR* InBufferEnd, FFeedbackContext* InWarn)
{
	return nullptr;
}

bool UPropertyAnimatorCorePresetFactory::FactoryCanImport(const FString& InFilename)
{
	return false;
}

#undef LOCTEXT_NAMESPACE
