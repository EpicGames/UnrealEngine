// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "PropertyAnimatorCorePresetFactory.generated.h"

class UPropertyAnimatorCorePresetBase;

UCLASS()
class UPropertyAnimatorCorePresetFactory : public UFactory
{
	GENERATED_BODY()

public:
	UPropertyAnimatorCorePresetFactory();

	//~ Begin UFactory
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn) override;
	virtual UObject* FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, const TCHAR* InType, const TCHAR*& InBuffer, const TCHAR* InBufferEnd, FFeedbackContext* InWarn) override;
	virtual bool FactoryCanImport(const FString& InFilename) override;
	//~ End UFactory

	TSubclassOf<UPropertyAnimatorCorePresetBase> NewPresetClass;
};
