// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyAnimatorCoreEditorModule.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Customizations/PropertyAnimatorCoreEditorContextTypeCustomization.h"
#include "Customizations/PropertyAnimatorCoreEditorDetailCustomization.h"
#include "Customizations/PropertyAnimatorCoreEditorManualStateTypeCustomization.h"
#include "Customizations/PropertyAnimatorCoreEditorSeedTypeCustomization.h"
#include "Customizations/PropertyAnimatorCoreEditorSequencerTimeSourceEvalResultTypeCustomization.h"
#include "ISequencerModule.h"
#include "Modules/ModuleManager.h"
#include "Properties/PropertyAnimatorCoreContext.h"
#include "PropertyEditorModule.h"
#include "Sequencer/MovieSceneAnimatorTrackEditor.h"
#include "TimeSources/PropertyAnimatorCoreManualTimeSource.h"
#include "TimeSources/PropertyAnimatorCoreSequencerTimeSource.h"

void FPropertyAnimatorCoreEditorModule::StartupModule()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomPropertyTypeLayout(RegisteredCustomizations.Add_GetRef(UPropertyAnimatorCoreContext::StaticClass()->GetFName()), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPropertyAnimatorCoreEditorContextTypeCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(RegisteredCustomizations.Add_GetRef(StaticEnum<EPropertyAnimatorCoreManualStatus>()->GetFName()), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPropertyAnimatorCoreEditorManualStateTypeCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(RegisteredCustomizations.Add_GetRef(FPropertyAnimatorCoreSequencerTimeSourceEvalResult::StaticStruct()->GetFName()), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPropertyAnimatorCoreEditorSequencerTimeSourceChannelTypeCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(RegisteredCustomizations.Add_GetRef(FIntProperty::StaticClass()->GetFName()), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPropertyAnimatorCoreEditorSeedTypeCustomization::MakeInstance), MakeShared<FPropertyAnimatorCoreEditorSeedTypeIdentifier>());

	PropertyModule.RegisterCustomClassLayout(RegisteredCustomizations.Add_GetRef(UPropertyAnimatorCoreBase::StaticClass()->GetFName()), FOnGetDetailCustomizationInstance::CreateStatic(&FPropertyAnimatorCoreEditorDetailCustomization::MakeInstance));

	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	AnimatorTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FMovieSceneAnimatorTrackEditor::CreateTrackEditor));
}

void FPropertyAnimatorCoreEditorModule::ShutdownModule()
{
	if (UObjectInitialized() && !IsEngineExitRequested())
	{
		if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
		{
			for (const FName& RegisteredCustomization : RegisteredCustomizations)
			{
				PropertyModule->UnregisterCustomPropertyTypeLayout(RegisteredCustomization);
				PropertyModule->UnregisterCustomClassLayout(RegisteredCustomization);
			}
		}

		RegisteredCustomizations.Empty();
		
		if (ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer"))
		{
			SequencerModule->UnRegisterTrackEditor(AnimatorTrackCreateEditorHandle);
			AnimatorTrackCreateEditorHandle.Reset();
		}
	}
}

IMPLEMENT_MODULE(FPropertyAnimatorCoreEditorModule, PropertyAnimatorCoreEditor)
