// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Modules/ModuleManager.h"
#include "UObject/NameTypes.h"

class FPropertyAnimatorCoreEditorModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

private:
	TArray<FName> RegisteredCustomizations;
	FDelegateHandle AnimatorTrackCreateEditorHandle;
};