// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimatorTrackEditor.h"

#include "Components/PropertyAnimatorCoreComponent.h"
#include "LevelSequence.h"
#include "Sequencer/MovieSceneAnimatorSection.h"

#define LOCTEXT_NAMESPACE "MovieSceneAnimatorTrackEditor"

FMovieSceneAnimatorTrackEditor::FOnAddAnimatorTrack FMovieSceneAnimatorTrackEditor::OnAddAnimatorTrack;
FMovieSceneAnimatorTrackEditor::FOnGetAnimatorTrackCount FMovieSceneAnimatorTrackEditor::OnGetAnimatorTrackCount;

FMovieSceneAnimatorTrackEditor::~FMovieSceneAnimatorTrackEditor()
{
	OnAddAnimatorTrack.RemoveAll(this);
	OnGetAnimatorTrackCount.RemoveAll(this);
}

FText FMovieSceneAnimatorTrackEditor::GetDisplayName() const
{
	return LOCTEXT("AnimatorTrackEditor_DisplayName", "Animator");
}

void FMovieSceneAnimatorTrackEditor::BuildAddTrackMenu(FMenuBuilder& InMenuBuilder)
{
	// Empty, do not allow the creation of new tracks, only allow through sequencer time source
}

TSharedPtr<SWidget> FMovieSceneAnimatorTrackEditor::BuildOutlinerEditWidget(const FGuid& InObjectBinding, UMovieSceneTrack* InTrack, const FBuildEditWidgetParams& InParams)
{
	// Empty, do not allow the creation of new sections, only one to rule them all
	return nullptr;
}

bool FMovieSceneAnimatorTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	const ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneAnimatorTrack::StaticClass()) : ETrackSupport::NotSupported;

	if (TrackSupported == ETrackSupport::NotSupported)
	{
		return false;
	}

	return InSequence->IsA(ULevelSequence::StaticClass());
}

void FMovieSceneAnimatorTrackEditor::BindDelegates()
{
	OnAddAnimatorTrack.AddSP(this, &FMovieSceneAnimatorTrackEditor::ExecuteAddTrack);
	OnGetAnimatorTrackCount.AddSP(this, &FMovieSceneAnimatorTrackEditor::GetTrackCount);
}

void FMovieSceneAnimatorTrackEditor::GetTrackCount(const TArray<UObject*>& InOwners, int32& OutCount) const
{
	const UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr)
	{
		return;
	}

	for (UObject* Owner : InOwners)
	{
		if (!Owner)
		{
			continue;
		}

		const FGuid ObjectBinding = SequencerPtr->GetHandleToObject(Owner, /** Create */false);

		if (!ObjectBinding.IsValid())
		{
			return;
		}

		for (const UMovieSceneTrack* Track : FocusedMovieScene->FindTracks(UMovieSceneAnimatorTrack::StaticClass(), ObjectBinding))
		{
			if (Track->IsA<UMovieSceneAnimatorTrack>())
			{
				OutCount++;
			}
		}
	}
}

bool FMovieSceneAnimatorTrackEditor::CanExecuteAddTrack() const
{
	return GetSequencer() && GetFocusedMovieScene();
}

void FMovieSceneAnimatorTrackEditor::ExecuteAddTrack(const TArray<UObject*>& InOwners)
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	if (!FocusedMovieScene || FocusedMovieScene->IsReadOnly())
	{
		return;
	}

	const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddAnimatorTrack", "Add Animator Track"));

	FocusedMovieScene->Modify();

	for (UObject* Owner : InOwners)
	{
		if (!Owner || !Owner->GetOuter())
		{
			continue;
		}

		const FGuid ObjectBinding = SequencerPtr->GetHandleToObject(Owner, /** Create */true);

		if (!ObjectBinding.IsValid() || FocusedMovieScene->FindSpawnable(ObjectBinding))
		{
			// we only want to add tracks for possessables
			continue;
		}

		if (!FocusedMovieScene->FindTrack<UMovieSceneAnimatorTrack>(ObjectBinding))
		{
			UMovieSceneAnimatorTrack* NewTrack = FocusedMovieScene->AddTrack<UMovieSceneAnimatorTrack>(ObjectBinding);

			if (const UPropertyAnimatorCoreBase* Animator = Cast<UPropertyAnimatorCoreBase>(Owner->GetOuter()))
			{
				NewTrack->SetDisplayName(FText::Format(LOCTEXT("MovieSceneAnimatorTrackName", "Animator {0} Track"), Animator->GetAnimatorMetadata()->DisplayName));
			}
			else
			{
				NewTrack->SetDisplayName(LOCTEXT("MovieSceneAnimatorComponentTrackName", "Animator Component Track"));
			}

			UMovieSceneAnimatorSection* NewSection = Cast<UMovieSceneAnimatorSection>(NewTrack->CreateNewSection());
			NewTrack->AddSection(*NewSection);
		}
	}

	SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}

#undef LOCTEXT_NAMESPACE
