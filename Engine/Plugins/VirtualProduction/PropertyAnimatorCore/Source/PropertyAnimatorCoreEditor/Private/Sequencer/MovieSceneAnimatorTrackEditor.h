// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KeyframeTrackEditor.h"
#include "Sequencer/MovieSceneAnimatorTrack.h"

class UMovieSceneSequence;

/** Animator track editor to add animator track and section */
class FMovieSceneAnimatorTrackEditor : public FKeyframeTrackEditor<UMovieSceneAnimatorTrack>
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAddAnimatorTrack, const TArray<UObject*>& /** InOwners */)
	static FOnAddAnimatorTrack OnAddAnimatorTrack;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGetAnimatorTrackCount, const TArray<UObject*>& /** InOwner */, int32& /** Count */)
	static FOnGetAnimatorTrackCount OnGetAnimatorTrackCount;

	FMovieSceneAnimatorTrackEditor(const TSharedRef<ISequencer>& InSequencer)
		: FKeyframeTrackEditor<UMovieSceneAnimatorTrack>(InSequencer)
	{}

	virtual ~FMovieSceneAnimatorTrackEditor() override;

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
	{
		TSharedRef<FMovieSceneAnimatorTrackEditor> TrackEditor = MakeShared<FMovieSceneAnimatorTrackEditor>(InSequencer);
		TrackEditor->BindDelegates();
		return TrackEditor;
	}

private:
	//~ Begin FMovieSceneTrackEditor
	virtual FText GetDisplayName() const override;
	virtual void BuildAddTrackMenu(FMenuBuilder& InMenuBuilder) override;
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& InObjectBinding, UMovieSceneTrack* InTrack, const FBuildEditWidgetParams& InParams) override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	//~ End FMovieSceneTrackEditor

	void BindDelegates();

	void GetTrackCount(const TArray<UObject*>& InOwners, int32& OutCount) const;

	bool CanExecuteAddTrack() const;
	void ExecuteAddTrack(const TArray<UObject*>& InOwners);
};
