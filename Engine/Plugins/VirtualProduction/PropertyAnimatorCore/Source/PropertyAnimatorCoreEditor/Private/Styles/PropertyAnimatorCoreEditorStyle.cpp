// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styles/PropertyAnimatorCoreEditorStyle.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleRegistry.h"

FPropertyAnimatorCoreEditorStyle::FPropertyAnimatorCoreEditorStyle()
	: FSlateStyleSet(UE_MODULE_NAME)
{
	const FVector2f Icon16x16(16.f, 16.f);

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);

	check(Plugin.IsValid());

	ContentRootDir = FPaths::Combine(Plugin->GetBaseDir(), TEXT("Content"));

	Set("ClassIcon.MovieSceneAnimatorTrack",        new IMAGE_BRUSH_SVG("EditorIcons/PropertyAnimatorCoreDefault", Icon16x16));
	Set("ClassIcon.PropertyAnimatorCoreComponent",  new IMAGE_BRUSH_SVG("EditorIcons/PropertyAnimatorCoreDefault", Icon16x16));
	Set("ClassIcon.PropertyAnimatorCoreBase",       new IMAGE_BRUSH_SVG("EditorIcons/PropertyAnimatorCoreDefault", Icon16x16));
	Set("PropertyControlIcon.Default",              new IMAGE_BRUSH_SVG("EditorIcons/PropertyAnimatorCoreDefault", Icon16x16));
	Set("PropertyControlIcon.Linked",               new IMAGE_BRUSH_SVG("EditorIcons/PropertyAnimatorCoreLinked",  Icon16x16));
	Set("PropertyControlIcon.Export",               new IMAGE_BRUSH_SVG("EditorIcons/Export",                      Icon16x16));
	Set("ManualTimeSourceControl.PlayForward",      new IMAGE_BRUSH_SVG("EditorIcons/Play",                        Icon16x16));
	Set("ManualTimeSourceControl.PlayBackward",     new IMAGE_BRUSH_SVG("EditorIcons/PlayReverse",                 Icon16x16));
	Set("ManualTimeSourceControl.Pause",            new IMAGE_BRUSH_SVG("EditorIcons/Pause",                       Icon16x16));
	Set("ManualTimeSourceControl.Stop",             new IMAGE_BRUSH_SVG("EditorIcons/Stop",                        Icon16x16));
	Set("ManualTimeSourceControl.Reset",            new IMAGE_BRUSH("EditorIcons/Reset",                           Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FPropertyAnimatorCoreEditorStyle::~FPropertyAnimatorCoreEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}