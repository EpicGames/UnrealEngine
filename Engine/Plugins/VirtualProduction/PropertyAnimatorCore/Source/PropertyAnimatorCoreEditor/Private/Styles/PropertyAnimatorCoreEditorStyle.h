// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FPropertyAnimatorCoreEditorStyle : public FSlateStyleSet
{
public:
	FPropertyAnimatorCoreEditorStyle();

	virtual ~FPropertyAnimatorCoreEditorStyle() override;

	static FPropertyAnimatorCoreEditorStyle& Get()
	{
		static FPropertyAnimatorCoreEditorStyle StyleSet;
		return StyleSet;
	}
};