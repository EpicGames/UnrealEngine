// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/PropertyAnimatorCoreEditorSubsystem.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Components/PropertyAnimatorCoreComponent.h"
#include "DetailRowMenuContext.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailTreeNode.h"
#include "Menus/PropertyAnimatorCoreEditorMenu.h"
#include "Menus/PropertyAnimatorCoreEditorMenuContext.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Styles/PropertyAnimatorCoreEditorStyle.h"
#include "Subsystems/PropertyAnimatorCoreSubsystem.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorCoreEditorSubsystem"

bool UPropertyAnimatorCoreEditorSubsystem::FillAnimatorMenu(UToolMenu* InMenu, const FPropertyAnimatorCoreEditorMenuContext& InContext, const FPropertyAnimatorCoreEditorMenuOptions& InOptions)
{
	if (!InMenu || InContext.IsEmpty())
	{
		return false;
	}

	LastMenuData = MakeShared<FPropertyAnimatorCoreEditorMenuData>(InContext, InOptions);

	FToolMenuSection* AnimatorSection = nullptr;
	if (InOptions.ShouldCreateSubMenu())
	{
		static const FName AnimatorSectionName("ContextAnimatorActions");

		AnimatorSection = InMenu->FindSection(AnimatorSectionName);
		if (!AnimatorSection)
		{
			AnimatorSection = &InMenu->AddSection(AnimatorSectionName
				, LOCTEXT("ContextAnimatorActions", "Animators Actions")
				, FToolMenuInsert(NAME_None, EToolMenuInsertType::First));
		}
	}

	if (InOptions.IsMenuType(EPropertyAnimatorCoreEditorMenuType::NewSimple) && InContext.ContainsAnyActor())
	{
		if (InOptions.ShouldCreateSubMenu())
		{
			AnimatorSection->AddSubMenu(
				TEXT("NewSimpleAnimatorMenu"),
				LOCTEXT("NewSimpleAnimatorMenu.Label", "Add Animators"),
				LOCTEXT("NewSimpleAnimatorMenu.Tooltip", "Add animators to the selection"),
				FNewToolMenuDelegate::CreateLambda(&UE::PropertyAnimatorCoreEditor::Menu::FillNewAnimatorSection, LastMenuData.ToSharedRef()));
		}
		else
		{
			UE::PropertyAnimatorCoreEditor::Menu::FillNewAnimatorSection(InMenu, LastMenuData.ToSharedRef());
		}
	}

	if (InOptions.IsMenuType(EPropertyAnimatorCoreEditorMenuType::NewAdvanced) && InContext.ContainsAnyActor())
	{
		if (InOptions.ShouldCreateSubMenu())
		{
			AnimatorSection->AddSubMenu(
				TEXT("NewAdvancedAnimatorMenu"),
				LOCTEXT("NewAdvancedAnimatorMenu.Label", "Add Animators"),
				LOCTEXT("NewAdvancedAnimatorMenu.Tooltip", "Add animators to the selection"),
				FNewToolMenuDelegate::CreateLambda(&UE::PropertyAnimatorCoreEditor::Menu::FillNewAnimatorSection, LastMenuData.ToSharedRef()));
		}
		else
		{
			UE::PropertyAnimatorCoreEditor::Menu::FillNewAnimatorSection(InMenu, LastMenuData.ToSharedRef());
		}
	}

	if (InOptions.IsMenuType(EPropertyAnimatorCoreEditorMenuType::Existing) && InContext.ContainsAnyProperty())
	{
		if (InOptions.ShouldCreateSubMenu())
		{
			AnimatorSection->AddSubMenu(
				TEXT("ExistingAnimatorMenu"),
				LOCTEXT("ExistingAnimatorMenu.Label", "Existing Animators"),
				LOCTEXT("ExistingAnimatorMenu.Tooltip", "Link or unlink selection to/from existing animators"),
				FNewToolMenuDelegate::CreateLambda(&UE::PropertyAnimatorCoreEditor::Menu::FillExistingAnimatorSection, LastMenuData.ToSharedRef()));
		}
		else
		{
			UE::PropertyAnimatorCoreEditor::Menu::FillExistingAnimatorSection(InMenu, LastMenuData.ToSharedRef());
		}
	}

	if (InOptions.IsMenuType(EPropertyAnimatorCoreEditorMenuType::Link) && InContext.ContainsAnyAnimator())
	{
		if (InOptions.ShouldCreateSubMenu())
		{
			AnimatorSection->AddSubMenu(
				TEXT("LinkAnimatorMenu"),
				LOCTEXT("LinkAnimatorMenu.Label", "Link Animators"),
				LOCTEXT("LinkAnimatorMenu.Tooltip", "Link selection to/from animators"),
				FNewToolMenuDelegate::CreateLambda(&UE::PropertyAnimatorCoreEditor::Menu::FillLinkAnimatorSection, LastMenuData.ToSharedRef()));
		}
		else
		{
			UE::PropertyAnimatorCoreEditor::Menu::FillLinkAnimatorSection(InMenu, LastMenuData.ToSharedRef());
		}
	}

	if (InOptions.IsMenuType(EPropertyAnimatorCoreEditorMenuType::Enable) && InContext.ContainsAnyDisabledAnimator())
	{
		if (InOptions.ShouldCreateSubMenu())
		{
			AnimatorSection->AddSubMenu(
				TEXT("EnableAnimatorMenu"),
				LOCTEXT("EnableAnimatorMenu.Label", "Enable Animators"),
				LOCTEXT("EnableAnimatorMenu.Tooltip", "Enable selected animators"),
				FNewToolMenuDelegate::CreateLambda(&UE::PropertyAnimatorCoreEditor::Menu::FillEnableAnimatorSection, LastMenuData.ToSharedRef()));
		}
		else
		{
			UE::PropertyAnimatorCoreEditor::Menu::FillEnableAnimatorSection(InMenu, LastMenuData.ToSharedRef());
		}
	}

	if (InOptions.IsMenuType(EPropertyAnimatorCoreEditorMenuType::Disable) && InContext.ContainsAnyEnabledAnimator())
	{
		if (InOptions.ShouldCreateSubMenu())
		{
			AnimatorSection->AddSubMenu(
				TEXT("DisableAnimatorMenu"),
				LOCTEXT("DisableAnimatorMenu.Label", "Disable Animators"),
				LOCTEXT("DisableAnimatorMenu.Tooltip", "Disable selected animators"),
				FNewToolMenuDelegate::CreateLambda(&UE::PropertyAnimatorCoreEditor::Menu::FillDisableAnimatorSection, LastMenuData.ToSharedRef()));
		}
		else
		{
			UE::PropertyAnimatorCoreEditor::Menu::FillDisableAnimatorSection(InMenu, LastMenuData.ToSharedRef());
		}
	}

	if (InOptions.IsMenuType(EPropertyAnimatorCoreEditorMenuType::Delete) && InContext.ContainsAnyComponentAnimator())
	{
		if (InOptions.ShouldCreateSubMenu())
		{
			AnimatorSection->AddSubMenu(
				TEXT("DeleteAnimatorMenu"),
				LOCTEXT("DeleteAnimatorMenu.Label", "Delete Animators"),
				LOCTEXT("DeleteAnimatorMenu.Tooltip", "Delete selected animators"),
				FNewToolMenuDelegate::CreateLambda(&UE::PropertyAnimatorCoreEditor::Menu::FillDeleteAnimatorSection, LastMenuData.ToSharedRef()));
		}
		else
		{
			UE::PropertyAnimatorCoreEditor::Menu::FillDeleteAnimatorSection(InMenu, LastMenuData.ToSharedRef());
		}
	}

	return false;
}

UPropertyAnimatorCorePresetBase* UPropertyAnimatorCoreEditorSubsystem::CreatePresetAsset(TSubclassOf<UPropertyAnimatorCorePresetBase> InPresetClass, const TArray<IPropertyAnimatorCorePresetable*>& InPresetables)
{
	UPropertyAnimatorCorePresetBase* NewPreset = nullptr;

	if (!InPresetClass.Get() || InPresetables.IsEmpty())
	{
		return NewPreset;
	}

	// Pick asset path and name
	FString PickedPath;
	FString PickedName;
	{
		TSharedPtr<SDlgPickAssetPath> DialogWidget = SNew(SDlgPickAssetPath)
		.Title(LOCTEXT("PickAssetsLocation", "Choose preset name and location"))
		.DefaultAssetPath(FText::FromString(TEXT("/PropertyAnimatorCore/Presets/NewPreset")))
		.AllowReadOnlyFolders(true);

		if (DialogWidget->ShowModal() != EAppReturnType::Ok)
		{
			PickedPath = TEXT("");
			return NewPreset;
		}

		PickedPath = DialogWidget->GetAssetPath().ToString();
		PickedName = DialogWidget->GetAssetName().ToString();
	}

	if (PickedPath.IsEmpty() || PickedName.IsEmpty())
	{
		return NewPreset;
	}

	// Find/create package
	UPackage* Package = CreatePackage(*(PickedPath + TEXT("/") + PickedName));

	if (!Package)
	{
		return nullptr;
	}

	NewPreset = NewObject<UPropertyAnimatorCorePresetBase>(Package, InPresetClass.Get(), FName(PickedName), RF_Public | RF_Standalone);
	NewPreset->CreatePreset(FName(PickedName), InPresetables);
	NewPreset->MarkPackageDirty();

	// Notify asset registry of new asset
	FAssetRegistryModule::AssetCreated(NewPreset);

	return NewPreset;
}

bool UPropertyAnimatorCoreEditorSubsystem::RegisterAnimatorCategory(const FPropertyAnimatorCoreEditorCategoryMetadata& InCategoryMetadata)
{
	if (InCategoryMetadata.Name.IsNone())
	{
		return false;
	}

	if (AnimatorCategories.Contains(InCategoryMetadata.Name))
	{
		return false;
	}

	AnimatorCategories.Emplace(InCategoryMetadata.Name, MakeShared<const FPropertyAnimatorCoreEditorCategoryMetadata>(InCategoryMetadata));

	return true;
}

TSharedPtr<const FPropertyAnimatorCoreEditorCategoryMetadata> UPropertyAnimatorCoreEditorSubsystem::FindAnimatorCategory(FName InCategoryIdentifier) const
{
	if (const TSharedRef<const FPropertyAnimatorCoreEditorCategoryMetadata>* Category = AnimatorCategories.Find(InCategoryIdentifier))
	{
		return *Category;
	}
	
	return nullptr;
}

UPropertyAnimatorCoreEditorSubsystem* UPropertyAnimatorCoreEditorSubsystem::Get()
{
	if (GEditor)
	{
		return GEditor->GetEditorSubsystem<UPropertyAnimatorCoreEditorSubsystem>();
	}
	return nullptr;
}

void UPropertyAnimatorCoreEditorSubsystem::RegisterDetailPanelCustomization()
{
	FPropertyEditorModule& PropertyEditor = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	OnGetGlobalRowExtensionHandle = PropertyEditor.GetGlobalRowExtensionDelegate().AddUObject(this, &UPropertyAnimatorCoreEditorSubsystem::OnGetGlobalRowExtension);
}

void UPropertyAnimatorCoreEditorSubsystem::UnregisterDetailPanelCustomization()
{
	FPropertyEditorModule& PropertyEditor = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditor.GetGlobalRowExtensionDelegate().Remove(OnGetGlobalRowExtensionHandle);
	OnGetGlobalRowExtensionHandle.Reset();
}

void UPropertyAnimatorCoreEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	// Initialize here, to setup ClassIcons if no other code uses it
	FPropertyAnimatorCoreEditorStyle::Get();

	// Register animator categories
	{
		const FPropertyAnimatorCoreEditorCategoryMetadata DefaultCategory
		{
			.Name = TEXT("Default"),
			.DisplayName = LOCTEXT("DefaultCategoryDisplayName", "Default")
		};

		RegisterAnimatorCategory(DefaultCategory);

		const FPropertyAnimatorCoreEditorCategoryMetadata PresetCategory
		{
			.Name = TEXT("Presets"),
			.DisplayName = LOCTEXT("PresetCategoryDisplayName", "Presets")
		};

		RegisterAnimatorCategory(PresetCategory);

		const FPropertyAnimatorCoreEditorCategoryMetadata TextCategory
		{
			.Name = TEXT("Text"),
			.DisplayName = LOCTEXT("TextCategoryDisplayName", "Text")
		};

		RegisterAnimatorCategory(TextCategory);

		const FPropertyAnimatorCoreEditorCategoryMetadata NumericCategory
		{
			.Name = TEXT("Numeric"),
			.DisplayName = LOCTEXT("NumericCategoryDisplayName", "Numeric")
		};

		RegisterAnimatorCategory(NumericCategory);
	}

	RegisterDetailPanelCustomization();
}

void UPropertyAnimatorCoreEditorSubsystem::Deinitialize()
{
	Super::Deinitialize();

	UnregisterDetailPanelCustomization();
}

void UPropertyAnimatorCoreEditorSubsystem::OnGetGlobalRowExtension(const FOnGenerateGlobalRowExtensionArgs& InArgs, TArray<FPropertyRowExtensionButton>& OutExtensions)
{
	TSharedPtr<IPropertyHandle> PropertyHandle = InArgs.PropertyHandle;
	if (!PropertyHandle.IsValid())
	{
		return;
	}

	// Extend context row menu
	ExtendPropertyRowContextMenu();

	TWeakPtr<IDetailTreeNode> OwnerTreeNodeWeak = InArgs.OwnerTreeNode;

	const FText Label = LOCTEXT("PropertyAnimatorCoreEditorExtension.Label", "Edit Animators");
	const FText Tooltip = LOCTEXT("PropertyAnimatorCoreEditorExtension.Tooltip", "Edit animators for this property");

	// Set custom icon when linked or not
	const TAttribute<FSlateIcon> Icon = MakeAttributeLambda([this, OwnerTreeNodeWeak, PropertyHandle]()
	{
		static const FSlateIcon ControlPropertyIcon(FPropertyAnimatorCoreEditorStyle::Get().GetStyleSetName(), "PropertyControlIcon.Default");
		static const FSlateIcon LinkedControlPropertyIcon(FPropertyAnimatorCoreEditorStyle::Get().GetStyleSetName(), "PropertyControlIcon.Linked");

		return IsControlPropertyLinked(OwnerTreeNodeWeak, PropertyHandle) ? LinkedControlPropertyIcon : ControlPropertyIcon;
	});

	const FUIAction Action = FUIAction(
		FExecuteAction::CreateUObject(this, &UPropertyAnimatorCoreEditorSubsystem::OnControlPropertyClicked, OwnerTreeNodeWeak, PropertyHandle),
		FCanExecuteAction::CreateUObject(this, &UPropertyAnimatorCoreEditorSubsystem::IsControlPropertySupported, OwnerTreeNodeWeak, PropertyHandle),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateUObject(this, &UPropertyAnimatorCoreEditorSubsystem::IsControlPropertyVisible, OwnerTreeNodeWeak, PropertyHandle)
	);

	FPropertyRowExtensionButton& AnimatePropertyButton = OutExtensions.AddDefaulted_GetRef();
	AnimatePropertyButton.Label = Label;
	AnimatePropertyButton.ToolTip = Tooltip;
	AnimatePropertyButton.Icon = Icon;
	AnimatePropertyButton.UIAction = Action;
}

void UPropertyAnimatorCoreEditorSubsystem::OnControlPropertyClicked(TWeakPtr<IDetailTreeNode> InOwnerTreeNode, TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	constexpr bool bFindMemberProperty = true;
	TArray<FPropertyAnimatorCoreData> Properties;
	if (!GetPropertiesFromHandle(InPropertyHandle, Properties, bFindMemberProperty))
	{
		return;
	}

	const TSharedPtr<SWindow> ActiveWindow = FSlateApplication::Get().GetActiveTopLevelWindow();
	if (!ActiveWindow)
	{
		return;
	}

	// Open context menu
	const FVector2D MenuLocation = FSlateApplication::Get().GetCursorPos() + FVector2D(0, 16);

	FSlateApplication::Get().PushMenu(
		ActiveWindow.ToSharedRef(),
		FWidgetPath(),
		GenerateContextMenuWidget(Properties),
		MenuLocation,
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
	);
}

bool UPropertyAnimatorCoreEditorSubsystem::IsControlPropertySupported(TWeakPtr<IDetailTreeNode> InOwnerTreeNode, TSharedPtr<IPropertyHandle> InPropertyHandle) const
{
	constexpr bool bFindMemberProperty = true;
	TArray<FPropertyAnimatorCoreData> Properties;
	if (!GetPropertiesFromHandle(InPropertyHandle, Properties, bFindMemberProperty))
	{
		return false;
	}

	const UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();
	if (!Subsystem)
	{
		return false;
	}

	for (FPropertyAnimatorCoreData& Property : Properties)
	{
		if (!Subsystem->IsPropertySupported(Property))
		{
			return false;
		}
	}

	return true;
}

bool UPropertyAnimatorCoreEditorSubsystem::IsControlPropertyVisible(TWeakPtr<IDetailTreeNode> InOwnerTreeNode, TSharedPtr<IPropertyHandle> InPropertyHandle) const
{
	return IsControlPropertyLinked(InOwnerTreeNode, InPropertyHandle);
}

bool UPropertyAnimatorCoreEditorSubsystem::IsControlPropertyLinked(TWeakPtr<IDetailTreeNode> InOwnerTreeNode, TSharedPtr<IPropertyHandle> InPropertyHandle) const
{
	constexpr bool bFindMemberProperty = true;
	TArray<FPropertyAnimatorCoreData> Properties;
	if (!GetPropertiesFromHandle(InPropertyHandle, Properties, bFindMemberProperty))
	{
		return false;
	}

	const UPropertyAnimatorCoreSubsystem* Subsystem = UPropertyAnimatorCoreSubsystem::Get();
	if (!Subsystem)
	{
		return false;
	}

	bool bAnimated = false;
	for (FPropertyAnimatorCoreData& Property : Properties)
	{
		for (const UPropertyAnimatorCoreBase* Animator : Subsystem->GetExistingAnimators(Property.GetOwningActor()))
		{
			if (Animator->IsPropertyLinked(Property))
			{
				bAnimated |= true;
			}
			else if (!Animator->GetInnerPropertiesLinked(Property).IsEmpty())
			{
				bAnimated |= true;
			}
			else
			{
				bAnimated |= false;
				break;
			}
		}

		if (!bAnimated)
		{
			break;
		}
	}

	return bAnimated;
}

TSharedRef<SWidget> UPropertyAnimatorCoreEditorSubsystem::GenerateContextMenuWidget(const TArray<FPropertyAnimatorCoreData>& InProperties)
{
	UToolMenus* Menus = UToolMenus::Get();

	check(Menus);

	static const FName AnimatorExtensionMenuName = TEXT("AnimatorExtensionMenu");

	if (!Menus->IsMenuRegistered(AnimatorExtensionMenuName))
	{
		UToolMenu* const DetailsViewExtensionMenu = Menus->RegisterMenu(AnimatorExtensionMenuName, NAME_None, EMultiBoxType::Menu);

		DetailsViewExtensionMenu->AddDynamicSection(
			TEXT("FillAnimatorExtensionSection")
			, FNewToolMenuDelegate::CreateUObject(this, &UPropertyAnimatorCoreEditorSubsystem::FillAnimatorExtensionSection)
		);
	}

	UPropertyAnimatorCoreEditorMenuContext* MenuContext = NewObject<UPropertyAnimatorCoreEditorMenuContext>();
	MenuContext->SetProperties(InProperties);

	const FToolMenuContext ToolMenuContext(MenuContext);
	return Menus->GenerateWidget(AnimatorExtensionMenuName, ToolMenuContext);
}

void UPropertyAnimatorCoreEditorSubsystem::ExtendPropertyRowContextMenu()
{
	UToolMenus* Menus = UToolMenus::Get();

	check(Menus);

	if (UToolMenu* ContextMenu = Menus->FindMenu(UE::PropertyEditor::RowContextMenuName))
	{
		ContextMenu->AddDynamicSection(
			TEXT("FillAnimatorRowContextSection")
			, FNewToolMenuDelegate::CreateUObject(this, &UPropertyAnimatorCoreEditorSubsystem::FillAnimatorRowContextSection));
	}
}

void UPropertyAnimatorCoreEditorSubsystem::FillAnimatorExtensionSection(UToolMenu* InToolMenu)
{
	if (!InToolMenu)
	{
		return;
	}

	const UPropertyAnimatorCoreEditorMenuContext* const Context = InToolMenu->FindContext<UPropertyAnimatorCoreEditorMenuContext>();

	if (!Context)
	{
		return;
	}

	const FPropertyAnimatorCoreEditorMenuContext MenuContext({}, Context->GetProperties());
	const FPropertyAnimatorCoreEditorMenuOptions MenuOptions(
		{
			EPropertyAnimatorCoreEditorMenuType::NewAdvanced
			, EPropertyAnimatorCoreEditorMenuType::Existing
		}
	);
	FillAnimatorMenu(InToolMenu, MenuContext, MenuOptions);
}

void UPropertyAnimatorCoreEditorSubsystem::FillAnimatorRowContextSection(UToolMenu* InToolMenu)
{
	if (!InToolMenu)
	{
		return;
	}

	// For context menu in details view
	const UDetailRowMenuContext* Context = InToolMenu->FindContext<UDetailRowMenuContext>();

	if (!Context || Context->PropertyHandles.IsEmpty())
	{
		return;
	}

	constexpr bool bFindMemberProperty = true;
	TArray<FPropertyAnimatorCoreData> Properties;
	for (const TSharedPtr<IPropertyHandle>& PropertyHandle : Context->PropertyHandles)
	{
		if (GetPropertiesFromHandle(PropertyHandle, Properties, bFindMemberProperty))
		{
			break;
		}

		Properties.Reset();
	}

	if (Properties.IsEmpty())
	{
		return;
	}

	const FPropertyAnimatorCoreEditorMenuContext MenuContext({}, Properties);
	const FPropertyAnimatorCoreEditorMenuOptions MenuOptions(
		{
			EPropertyAnimatorCoreEditorMenuType::NewAdvanced
			, EPropertyAnimatorCoreEditorMenuType::Existing
		}
	);
	FillAnimatorMenu(InToolMenu, MenuContext, MenuOptions);
}

bool UPropertyAnimatorCoreEditorSubsystem::GetPropertiesFromHandle(const TSharedPtr<IPropertyHandle>& InPropertyHandle, TArray<FPropertyAnimatorCoreData>& OutProperties, bool bInFindMemberProperty) const
{
	if (!InPropertyHandle.IsValid() || !InPropertyHandle->IsValidHandle())
	{
		return false;
	}

	const FProperty* Property = InPropertyHandle->GetProperty();
	if (!Property)
	{
		return false;
	}

	TArray<UObject*> Owners;
	InPropertyHandle->GetOuterObjects(Owners);

	if (Owners.IsEmpty() || !IsValid(Owners[0]))
	{
		return false;
	}

	if (Owners[0]->IsA<UPropertyAnimatorCoreComponent>() || Owners[0]->GetTypedOuter<UPropertyAnimatorCoreComponent>())
	{
		return false;
	}

	const FString OwnerPath = Owners[0]->GetPathName();
	if (OwnerPath.IsEmpty())
	{
		return false;
	}

	const FString PropertyPath = InPropertyHandle->GeneratePathToProperty();
	if (PropertyPath.IsEmpty())
	{
		return false;
	}

	// Climbs up the tree to find a member property handle that is a direct child of an object property
	TFunction<TSharedPtr<IPropertyHandle>(TSharedPtr<IPropertyHandle>, bool)> FindMemberProperty = [&FindMemberProperty](TSharedPtr<IPropertyHandle> InHandle, bool bInRecurse)->TSharedPtr<IPropertyHandle>
	{
		if (!InHandle.IsValid() || !InHandle->IsValidHandle() || !InHandle->GetProperty())
		{
			return nullptr;
		}

		const TSharedPtr<IPropertyHandle> ParentHandle = InHandle->GetParentHandle();
		if (ParentHandle.IsValid() && ParentHandle->GetProperty() && !ParentHandle->GetProperty()->IsA<FObjectProperty>())
		{
			return bInRecurse ? FindMemberProperty(ParentHandle, bInRecurse) : nullptr;
		}

		return InHandle;
	};

	const TSharedPtr<IPropertyHandle> MemberPropertyHandle = FindMemberProperty(InPropertyHandle, bInFindMemberProperty);

	if (!MemberPropertyHandle.IsValid() || !MemberPropertyHandle->IsValidHandle())
	{
		return false;
	}

	FPropertyAnimatorCoreData PropertyData(Owners[0], MemberPropertyHandle->GetProperty(), nullptr);

	// We need a setter to control a property
	if (!PropertyData.HasSetter())
	{
		return false;
	}

	OutProperties.Emplace(PropertyData);
	for (int32 Index = 1; Index < Owners.Num(); Index++)
	{
		if (IsValid(Owners[Index]))
		{
			OutProperties.Emplace(FPropertyAnimatorCoreData(Owners[Index], MemberPropertyHandle->GetProperty(), nullptr));
		}
	}

	return PropertyData.GetOwningActor() != nullptr;
}

#undef LOCTEXT_NAMESPACE
