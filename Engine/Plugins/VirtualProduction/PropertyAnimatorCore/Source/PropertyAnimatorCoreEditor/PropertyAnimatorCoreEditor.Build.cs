// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PropertyAnimatorCoreEditor : ModuleRules
{
    public PropertyAnimatorCoreEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
	        new string[]
	        {
		        "Core"
	        }
        );

        PrivateDependencyModuleNames.AddRange(
	        new string[]
	        {
		        "ApplicationCore",
				"AssetDefinition",
		        "CoreUObject",
		        "EditorSubsystem",
		        "EditorWidgets",
		        "Engine",
		        "InputCore",
				"LevelSequence",
				"MovieScene",
				"MovieSceneTools",
		        "OperatorStackEditor",
		        "Projects",
				"PropertyAnimatorCore",
				"PropertyEditor",
				"Sequencer",
		        "SlateCore",
		        "Slate",
		        "ToolMenus",
		        "UnrealEd"
	        }
        );

		ShortName = "PropAnimCoreEd";
    }
}