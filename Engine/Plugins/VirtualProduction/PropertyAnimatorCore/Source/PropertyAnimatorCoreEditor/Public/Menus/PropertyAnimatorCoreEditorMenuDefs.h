// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Properties/PropertyAnimatorCoreData.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class UPropertyAnimatorCoreBase;
class UPropertyAnimatorCoreComponent;
class UObject;

/** Menu type available */
enum class EPropertyAnimatorCoreEditorMenuType : uint8
{
	/** Menu to add new animator on selected actors */
	NewSimple = 1 << 0,
	/** Menu to add new animator on selected actors, with submenus for presets and properties */
	NewAdvanced = 1 << 1,
	/** Menu to link or unlink properties/preset to/from existing animators */
	Existing = 1 << 2,
	/** Menu to link properties/preset to existing animator */
	Link = 1 << 3,
	/** Menu to delete animators */
	Delete = 1 << 4,
	/** Menu to enable animators */
	Enable = 1 << 5,
	/** Menu to disable animators */
	Disable = 1 << 6
};

/** Context for the menu builder, you can reuse it for multiple menus types */
struct PROPERTYANIMATORCOREEDITOR_API FPropertyAnimatorCoreEditorMenuContext
{
	FPropertyAnimatorCoreEditorMenuContext() {}
	explicit FPropertyAnimatorCoreEditorMenuContext(const TSet<UObject*>& InObjects, const TArray<FPropertyAnimatorCoreData>& InProperties);

	const TSet<AActor*>& GetActors() const;
	const TSet<FPropertyAnimatorCoreData>& GetProperties() const;
	const TSet<UPropertyAnimatorCoreComponent*>& GetComponents() const;
	const TSet<UPropertyAnimatorCoreBase*>& GetAnimators() const;
	TSet<UPropertyAnimatorCoreBase*> GetDisabledAnimators() const;
	TSet<UPropertyAnimatorCoreBase*> GetEnabledAnimators() const;
	UWorld* GetWorld() const;

	bool IsEmpty() const;
	bool ContainsAnyProperty() const;
	bool ContainsAnyActor() const;
	bool ContainsAnyAnimator() const;
	bool ContainsAnyComponent() const;
	bool ContainsAnyDisabledAnimator() const;
	bool ContainsAnyEnabledAnimator() const;
	bool ContainsAnyComponentAnimator() const;

protected:
	bool ContainsAnimatorState(bool bInState) const;
	TSet<UPropertyAnimatorCoreBase*> GetStateAnimators(bool bInState) const;

	TSet<FPropertyAnimatorCoreData> ContextProperties;
	TSet<AActor*> ContextActors;
	TSet<UPropertyAnimatorCoreComponent*> ContextComponents;
	TSet<UPropertyAnimatorCoreBase*> ContextAnimators;
};

/** Menu builder options */
struct PROPERTYANIMATORCOREEDITOR_API FPropertyAnimatorCoreEditorMenuOptions
{
	FPropertyAnimatorCoreEditorMenuOptions() {}
	explicit FPropertyAnimatorCoreEditorMenuOptions(const TSet<EPropertyAnimatorCoreEditorMenuType>& InMenus);
	explicit FPropertyAnimatorCoreEditorMenuOptions(uint8 InMenus)
		: MenuTypes(InMenus)
	{}

	FPropertyAnimatorCoreEditorMenuOptions& CreateSubMenu(bool bInCreateSubMenu);
	FPropertyAnimatorCoreEditorMenuOptions& UseTransact(bool bInUseTransact);

	bool IsMenuType(EPropertyAnimatorCoreEditorMenuType InMenuType) const;
	bool ShouldTransact() const { return bUseTransact; }
	bool ShouldCreateSubMenu() const { return bCreateSubMenu; }

protected:
	/** What type of menu should be generated */
	uint8 MenuTypes = 0;

	/** Create a transaction for actions performed using the menu */
	bool bUseTransact = true;

	/** Creates the section inside a submenu */
	bool bCreateSubMenu = false;
};

/** Used internally to group menu data together */
struct FPropertyAnimatorCoreEditorMenuData
{
	FPropertyAnimatorCoreEditorMenuData(const FPropertyAnimatorCoreEditorMenuContext& InContext, const FPropertyAnimatorCoreEditorMenuOptions& InOptions)
		: Context(InContext)
		, Options(InOptions)
	{}

	const FPropertyAnimatorCoreEditorMenuContext& GetContext() const
	{
		return Context;
	}

	const FPropertyAnimatorCoreEditorMenuOptions& GetOptions() const
	{
		return Options;
	}

	void SetLastCreatedAnimator(UPropertyAnimatorCoreBase* InAnimator);
	UPropertyAnimatorCoreBase* GetLastCreatedAnimator() const;

	void SetLastCreatedAnimators(const TSet<UPropertyAnimatorCoreBase*>& InAnimators);
	TSet<UPropertyAnimatorCoreBase*> GetLastCreatedAnimators() const;

	bool ContainsAnyLastCreatedAnimator() const;

protected:
	const FPropertyAnimatorCoreEditorMenuContext Context;
	const FPropertyAnimatorCoreEditorMenuOptions Options;

	/** Store last animators created to quickly link properties after creating it */
	TSet<TWeakObjectPtr<UPropertyAnimatorCoreBase>> LastCreatedAnimators;
};