// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "Menus/PropertyAnimatorCoreEditorMenuDefs.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "PropertyAnimatorCoreEditorSubsystem.generated.h"

class IDetailTreeNode;
class IPropertyAnimatorCorePresetable;
class IPropertyHandle;
class SPropertyAnimatorCoreEditorEditPanel;
class SWidget;
class UPropertyAnimatorCoreBase;
class UPropertyAnimatorCoreEditorMenuContext;
class UPropertyAnimatorCorePresetBase;
class UToolMenu;
struct FPropertyAnimatorCoreData;
struct FPropertyAnimatorCoreEditorEditPanelOptions;
struct FOnGenerateGlobalRowExtensionArgs;
struct FPropertyRowExtensionButton;

struct FPropertyAnimatorCoreEditorCategoryMetadata
{
	FName Name;
	FText DisplayName;
};

/** Singleton class that handles editor operations for property animators */
UCLASS()
class UPropertyAnimatorCoreEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	/** Get this subsystem instance */
	PROPERTYANIMATORCOREEDITOR_API static UPropertyAnimatorCoreEditorSubsystem* Get();

	//~ Begin UEditorSubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End UEditorSubsystem

	/** Fills a menu based on context objects and menu options */
	PROPERTYANIMATORCOREEDITOR_API bool FillAnimatorMenu(UToolMenu* InMenu, const FPropertyAnimatorCoreEditorMenuContext& InContext, const FPropertyAnimatorCoreEditorMenuOptions& InOptions);

	/** Creates a preset asset for an item */
	PROPERTYANIMATORCOREEDITOR_API UPropertyAnimatorCorePresetBase* CreatePresetAsset(TSubclassOf<UPropertyAnimatorCorePresetBase> InPresetClass, const TArray<IPropertyAnimatorCorePresetable*>& InPresetables);

	/** Register a category for animator */
	PROPERTYANIMATORCOREEDITOR_API bool RegisterAnimatorCategory(const FPropertyAnimatorCoreEditorCategoryMetadata& InCategoryMetadata);

	/** Finds a registered category for animator */
	PROPERTYANIMATORCOREEDITOR_API TSharedPtr<const FPropertyAnimatorCoreEditorCategoryMetadata> FindAnimatorCategory(FName InCategoryIdentifier) const;

protected:
	/** Setup details panel button customization */
	void RegisterDetailPanelCustomization();

	/** Removes details panel button customization */
	void UnregisterDetailPanelCustomization();

	/** Extends the row extension from the details panel to display a button for each property row */
	void OnGetGlobalRowExtension(const FOnGenerateGlobalRowExtensionArgs& InArgs, TArray<FPropertyRowExtensionButton>& OutExtensions);

	/** Called when user press the control property button in details panel */
	void OnControlPropertyClicked(TWeakPtr<IDetailTreeNode> InOwnerTreeNode, TSharedPtr<IPropertyHandle> InPropertyHandle);

	/** Checks whether any controller supports that property */
	bool IsControlPropertySupported(TWeakPtr<IDetailTreeNode> InOwnerTreeNode, TSharedPtr<IPropertyHandle> InPropertyHandle) const;

	/** Checks whether the property control window is opened to display the icon */
	bool IsControlPropertyVisible(TWeakPtr<IDetailTreeNode> InOwnerTreeNode, TSharedPtr<IPropertyHandle> InPropertyHandle) const;

	/** Checks whether any controller is linked to that property */
	bool IsControlPropertyLinked(TWeakPtr<IDetailTreeNode> InOwnerTreeNode, TSharedPtr<IPropertyHandle> InPropertyHandle) const;

	/** Creates the context menu to display when selecting the animator icon */
	TSharedRef<SWidget> GenerateContextMenuWidget(const TArray<FPropertyAnimatorCoreData>& InProperties);

	/** Extend the context menu on each property row in details panel with additional entries */
	void ExtendPropertyRowContextMenu();

	/** Fills the animator details view extension menu */
	void FillAnimatorExtensionSection(UToolMenu* InToolMenu);

	/** Fills the animator details view row context menu */
	void FillAnimatorRowContextSection(UToolMenu* InToolMenu);

	/** Extracts property data along with owners from a property handle */
	bool GetPropertiesFromHandle(const TSharedPtr<IPropertyHandle>& InPropertyHandle, TArray<FPropertyAnimatorCoreData>& OutProperties, bool bInFindMemberProperty = false) const;

	FDelegateHandle OnGetGlobalRowExtensionHandle;

	TWeakPtr<SPropertyAnimatorCoreEditorEditPanel> PropertyControllerPanelWeak;

	TSharedPtr<FPropertyAnimatorCoreEditorMenuData> LastMenuData;

	TMap<FName, TSharedRef<const FPropertyAnimatorCoreEditorCategoryMetadata>> AnimatorCategories;
};